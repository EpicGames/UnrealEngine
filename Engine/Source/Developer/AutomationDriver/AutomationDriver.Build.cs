// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AutomationDriver : ModuleRules
{
    public AutomationDriver(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
				"ApplicationCore",
                "InputCore",
                "Json",
                "Slate",
                "SlateCore",
            }
        );
    }
}
