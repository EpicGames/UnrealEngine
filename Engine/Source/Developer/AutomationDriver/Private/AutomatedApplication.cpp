// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedApplication.h"
#include "PassThroughMessageHandler.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"

class FAutomatedCursor
	: public ICursor
{
public:

	virtual ~FAutomatedCursor()
	{ }

	virtual FVector2D GetPosition() const override
	{
		if (bAllowMessageHandling)
		{
			if (RealCursor.IsValid())
			{
				if (!bOverrideRealCursor)
				{
					FakePosition = RealCursor->GetPosition();
				}
			}
			else
			{
				FakePosition = FVector2D::ZeroVector;
			}
		}

		return FakePosition;
	}

	virtual void SetPosition(const int32 X, const int32 Y) override
	{
		FakePosition = FVector2D(X, Y);

		if (bAllowMessageHandling)
		{
			if (RealCursor.IsValid())
			{
				RealCursor->SetPosition(X, Y);
			}
		}
	}

	virtual void SetType(const EMouseCursor::Type InNewCursor) override
	{
		FakeMouseType = InNewCursor;

		if (bAllowMessageHandling)
		{
			if (RealCursor.IsValid())
			{
				RealCursor->SetType(InNewCursor);
			}
		}
	}

	virtual EMouseCursor::Type GetType() const override
	{
		if (bAllowMessageHandling)
		{
			if (RealCursor.IsValid())
			{
				FakeMouseType = RealCursor->GetType();
			}
			else
			{
				FakeMouseType = EMouseCursor::Default;
			}
		}

		return FakeMouseType;
	}

	virtual void GetSize(int32& Width, int32& Height) const override
	{
		if (bAllowMessageHandling)
		{
			if (RealCursor.IsValid())
			{
				RealCursor->GetSize(FakeSizeWidth, FakeSizeHeight);
			}
		}

		Width = FakeSizeWidth;
		Height = FakeSizeHeight;
	}

	virtual void Show(bool bShow) override
	{
		if (bAllowMessageHandling)
		{
			if (RealCursor.IsValid())
			{
				RealCursor->Show(bShow);
			}
		}
	}

	virtual void Lock(const RECT* const Bounds) override
	{
		if (bAllowMessageHandling)
		{
			if (RealCursor.IsValid())
			{
				RealCursor->Lock(Bounds);
			}
		}
	}

	virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override
	{
		if (bAllowMessageHandling)
		{
			if (RealCursor.IsValid())
			{
				RealCursor->SetTypeShape(InCursorType, CursorHandle);
			}
		}
	}

public:

	bool IsHandlingMessages() const
	{
		return bAllowMessageHandling;
	}

	void SetAllowMessageHandling(bool bValue)
	{
		bAllowMessageHandling = bValue;
	}

	void SetOverrideRealCursorCoordinates(bool bOverride)
	{
		bOverrideRealCursor = bOverride;
	}

	FAutomatedCursor(const TSharedPtr<ICursor>& InRealCursor)
		: RealCursor(InRealCursor)
		, bAllowMessageHandling(false)
		, bOverrideRealCursor(false)
	{
		if (RealCursor.IsValid())
		{
			FakePosition = RealCursor->GetPosition();
			FakeMouseType = RealCursor->GetType();
			RealCursor->GetSize(FakeSizeWidth, FakeSizeHeight);
		}
		else
		{
			FakePosition = FVector2D::ZeroVector;
			FakeMouseType = EMouseCursor::Default;
			FakeSizeHeight = 5;
			FakeSizeWidth = 5;
		}
	}

private:

	const TSharedPtr<ICursor> RealCursor;

	bool bAllowMessageHandling;
	bool bOverrideRealCursor;

	mutable FVector2D FakePosition;
	mutable EMouseCursor::Type FakeMouseType;
	mutable int32 FakeSizeWidth;
	mutable int32 FakeSizeHeight;
};

//disable warnings from overriding the deprecated forcefeedback.  
//calls to the deprecated function will still generate warnings.
PRAGMA_DISABLE_DEPRECATION_WARNINGS

class FAutomatedApplicationImpl
	: public FAutomatedApplication
{
public:

	virtual ~FAutomatedApplicationImpl()
	{ }

	virtual void AllowPlatformMessageHandling() override
	{
		if (PassThroughMessageHandler.IsValid())
		{
			PassThroughMessageHandler->SetAllowMessageHandling(true);
		}

		if (AutomatedCursor.IsValid())
		{
			AutomatedCursor->SetAllowMessageHandling(true);
		}
	}

	virtual void DisablePlatformMessageHandling() override
	{
		if (PassThroughMessageHandler.IsValid())
		{
			PassThroughMessageHandler->SetAllowMessageHandling(false);
		}

		if (AutomatedCursor.IsValid())
		{
			AutomatedCursor->SetAllowMessageHandling(false);
		}
	}

	virtual bool IsHandlingMessages() const override
	{
		if (!PassThroughMessageHandler.IsValid() || !PassThroughMessageHandler->IsHandlingMessages())
		{
			return false;
		}

		if (!AutomatedCursor.IsValid() || !AutomatedCursor->IsHandlingMessages())
		{
			return false;
		}

		return true;
	}

	virtual void SetOverrideRealCursorCoordinates(bool bOverride) override
	{
		if (AutomatedCursor.IsValid())
		{
			AutomatedCursor->SetOverrideRealCursorCoordinates(bOverride);
		}
	}

	virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override
	{
		RealMessageHandler = InMessageHandler;
		PassThroughMessageHandler = PassThroughMessageHandlerFactory->Create(InMessageHandler);

		MessageHandler = PassThroughMessageHandler.ToSharedRef();
		RealApplication->SetMessageHandler(MessageHandler);
	}

	virtual void PollGameDeviceState(const float TimeDelta) override
	{
		RealApplication->PollGameDeviceState(TimeDelta);
	}

	virtual void PumpMessages(const float TimeDelta) override
	{
		RealApplication->PumpMessages(TimeDelta);
	}

	virtual void ProcessDeferredEvents(const float TimeDelta) override
	{
		RealApplication->ProcessDeferredEvents(TimeDelta);
	}

	virtual void Tick(const float TimeDelta) override
	{
		RealApplication->Tick(TimeDelta);
	}

	virtual TSharedRef<FGenericWindow> MakeWindow() override
	{
		return RealApplication->MakeWindow();
	}

	virtual void InitializeWindow(const TSharedRef<FGenericWindow>& Window, const TSharedRef<FGenericWindowDefinition>& Definition, const TSharedPtr<FGenericWindow>& Parent, const bool bShowImmediately) override
	{
		RealApplication->InitializeWindow(Window, Definition, Parent, bShowImmediately);
	}

	virtual void SetCapture(const TSharedPtr<FGenericWindow>& Window) override
	{
		if (IsHandlingMessages())
		{
			FakeCapture = Window;
		}

		RealApplication->SetCapture(Window);
	}

	virtual void* GetCapture(void) const override
	{
		if (IsHandlingMessages())
		{
			return (void*)FakeCapture.Get();
		}

		return RealApplication->GetCapture();
	}

	virtual FModifierKeysState GetModifierKeys() const override
	{
		if (IsHandlingMessages())
		{
			return FakeModifierKeys;
		}

		return RealApplication->GetModifierKeys();
	}

	virtual void SetFakeModifierKeys(FModifierKeysState Value) override
	{
		FakeModifierKeys = Value;
	}

	TSharedPtr<SWindow> InternalGetWindowUnderCursor() const
	{
		if (AutomatedCursor.IsValid())
		{
			TArray<TSharedRef<SWindow>> VisibleWindows;
			FSlateApplication::Get().GetAllVisibleWindowsOrdered(VisibleWindows);

			for (int32 Index = VisibleWindows.Num() - 1; Index >= 0; Index--)
			{
				if (VisibleWindows[Index]->GetRectInScreen().ContainsPoint(AutomatedCursor->GetPosition()))
				{
					return VisibleWindows[Index];
				}
			}
		}

		return nullptr;
	}

	virtual bool IsCursorDirectlyOverSlateWindow() const override
	{
		if (IsHandlingMessages())
		{
			return InternalGetWindowUnderCursor().IsValid();
		}

		return RealApplication->IsCursorDirectlyOverSlateWindow();
	}

	virtual TSharedPtr<FGenericWindow> GetWindowUnderCursor() override
	{
		if (IsHandlingMessages())
		{
			TSharedPtr<SWindow> Window = InternalGetWindowUnderCursor();
			if (Window.IsValid())
			{
				return Window->GetNativeWindow();
			}

			return nullptr;
		}

		return RealApplication->GetWindowUnderCursor();
	}

	virtual void SetHighPrecisionMouseMode(const bool bEnable, const TSharedPtr<FGenericWindow>& Window) override
	{
		RealApplication->SetHighPrecisionMouseMode(bEnable, Window);
	}

	virtual bool IsUsingHighPrecisionMouseMode() const override
	{
		return RealApplication->IsUsingHighPrecisionMouseMode();
	}

	virtual bool IsUsingTrackpad() const override
	{
		return RealApplication->IsUsingTrackpad();
	}

	virtual bool IsMouseAttached() const override
	{
		return RealApplication->IsMouseAttached();
	}

	virtual void RegisterConsoleCommandListener(const FOnConsoleCommandListener& Listener) override
	{
		RealApplication->RegisterConsoleCommandListener(Listener);
	}

	virtual void AddPendingConsoleCommand(const FString& InCommand) override
	{
		RealApplication->AddPendingConsoleCommand(InCommand);
	}

	virtual FPlatformRect GetWorkArea(const FPlatformRect& CurrentWindow) const override
	{
		return RealApplication->GetWorkArea(CurrentWindow);
	}

	virtual bool TryCalculatePopupWindowPosition(const FPlatformRect& Anchor, const FVector2D& Size, const FVector2D& ProposedPlacement, const EPopUpOrientation::Type Orientation, FVector2D* const OutCalculatedPopUpPosition) const override
	{
		return RealApplication->TryCalculatePopupWindowPosition(Anchor, Size, ProposedPlacement, Orientation, OutCalculatedPopUpPosition);
	}

	virtual void GetInitialDisplayMetrics(FDisplayMetrics& OutDisplayMetrics) const override
	{
		RealApplication->GetInitialDisplayMetrics(OutDisplayMetrics);
	}

	virtual EWindowTitleAlignment::Type GetWindowTitleAlignment() const override
	{
		return RealApplication->GetWindowTitleAlignment();
	}

	virtual EWindowTransparency GetWindowTransparencySupport() const override
	{
		return RealApplication->GetWindowTransparencySupport();
	}

	virtual void DestroyApplication() override
	{
		RealApplication->DestroyApplication();
	}

	virtual IInputInterface* GetInputInterface() override
	{
		return RealApplication->GetInputInterface();
	}

	virtual ITextInputMethodSystem* GetTextInputMethodSystem() override
	{
		return RealApplication->GetTextInputMethodSystem();
	}

	virtual void SendAnalytics(IAnalyticsProvider* Provider) override
	{
		RealApplication->SendAnalytics(Provider);
	}

	virtual bool SupportsSystemHelp() const override
	{
		return RealApplication->SupportsSystemHelp();
	}

	virtual void ShowSystemHelp() override
	{
		RealApplication->ShowSystemHelp();
	}

	virtual bool ApplicationLicenseValid(FPlatformUserId PlatformUser = PLATFORMUSERID_NONE) override
	{
		return RealApplication->ApplicationLicenseValid(PlatformUser);
	}

private:

	FAutomatedApplicationImpl(
		const TSharedRef<GenericApplication>& InPlatformApplication,
		const TSharedRef<IPassThroughMessageHandlerFactory>& InPassThroughMessageHandlerFactory)
		: FAutomatedApplication(MakeShareable(new FAutomatedCursor(InPlatformApplication->Cursor)))
		, RealApplication(InPlatformApplication)
		, PassThroughMessageHandlerFactory(InPassThroughMessageHandlerFactory)
	{
		AutomatedCursor = StaticCastSharedPtr<FAutomatedCursor>(Cursor);
	}

private:

	const TSharedRef<GenericApplication> RealApplication;
	const TSharedRef<IPassThroughMessageHandlerFactory> PassThroughMessageHandlerFactory;

	TSharedPtr<FPassThroughMessageHandler> PassThroughMessageHandler;
	TSharedPtr<FAutomatedCursor> AutomatedCursor;
	TSharedPtr<FGenericWindow> FakeCapture;
	FModifierKeysState FakeModifierKeys;

	friend FAutomatedApplicationFactory;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

TSharedRef<FAutomatedApplication> FAutomatedApplicationFactory::Create(
	const TSharedRef<GenericApplication>& PlatformApplication,
	const TSharedRef<IPassThroughMessageHandlerFactory>& PassThroughMessageHandlerFactory)
{
	return MakeShareable(new FAutomatedApplicationImpl(
		PlatformApplication,
		PassThroughMessageHandlerFactory));
}

