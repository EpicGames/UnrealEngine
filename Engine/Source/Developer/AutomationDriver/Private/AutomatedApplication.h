// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/ICursor.h"

class GenericApplication;
class FGenericApplicationMessageHandler;
class IPassThroughMessageHandlerFactory;

class FAutomatedApplication
	: public GenericApplication
{
public:

	TSharedRef<FGenericApplicationMessageHandler> GetRealMessageHandler() { return RealMessageHandler.ToSharedRef(); }

	virtual void AllowPlatformMessageHandling() = 0;
	virtual void DisablePlatformMessageHandling() = 0;
	virtual bool IsHandlingMessages() const = 0;
	virtual void SetOverrideRealCursorCoordinates(bool bOverrideRealCursor) = 0;

	virtual void SetFakeModifierKeys(FModifierKeysState Value) = 0;

	FAutomatedApplication(
		const TSharedPtr<ICursor>& InCursor)
		: GenericApplication(InCursor)
	{ }

protected:

	TSharedPtr<FGenericApplicationMessageHandler> RealMessageHandler;

private:

};

class FAutomatedApplicationFactory
{
public:

	static TSharedRef<FAutomatedApplication> Create(
		const TSharedRef<GenericApplication>& PlatformApplication,
		const TSharedRef<IPassThroughMessageHandlerFactory>& PassThroughMessageHandlerFactory);
};