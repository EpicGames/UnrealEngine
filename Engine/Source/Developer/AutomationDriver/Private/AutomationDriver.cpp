// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationDriver.h"
#include "AutomationDriverLogging.h"
#include "IApplicationElement.h"
#include "IDriverSequence.h"

#include "AutomatedApplication.h"
#include "DriverSequence.h"
#include "DriverElement.h"
#include "DriverConfiguration.h"
#include "WaitUntil.h"

#include "Async/Async.h"
#include "Framework/Application/SlateApplication.h"

FAsyncAutomationDriver::~FAsyncAutomationDriver()
{
	Application->SetFakeModifierKeys(FModifierKeysState());
}

bool FAsyncAutomationDriver::PinActionSequence(const TSharedPtr<IAsyncActionSequence, ESPMode::ThreadSafe>& Sequence)
{
	if (PinnedSequence && PinnedSequence->IsExecuting())
	{
		FAutomationDriverLogging::CannotExecuteMultipleActionSequencesInParallel();
		return false;
	}

	PinnedSequence = Sequence;
	return true;
}

bool FAsyncAutomationDriver::UnpinActionSequence(const TSharedPtr<IAsyncActionSequence, ESPMode::ThreadSafe>& Sequence)
{
	if (PinnedSequence != Sequence)
	{
		FAutomationDriverLogging::CannotUnpinActionSequenceIfNotPinned();
		return false;
	}

	if (PinnedSequence && PinnedSequence->IsExecuting())
	{
		FAutomationDriverLogging::CannotUnpinActionSequenceIfExecuting();
		return false;
	}

	PinnedSequence.Reset();
	return true;
}

TAsyncResult<bool> FAsyncAutomationDriver::Wait(FTimespan Timespan)
{
	TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = CreateSequence();
	Sequence->Actions().Wait(Timespan);
	return Sequence->Perform();
}

TAsyncResult<bool> FAsyncAutomationDriver::Wait(const FDriverWaitDelegate& Delegate)
{
	TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = CreateSequence();
	Sequence->Actions().Wait(Delegate);
	return Sequence->Perform();
}

TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> FAsyncAutomationDriver::CreateSequence()
{
	return FAsyncDriverSequenceFactory::Create(FAsyncActionSequenceFactory::Create(
		SharedThis(this),
		Application));
}

TAsyncResult<FVector2D> FAsyncAutomationDriver::GetCursorPosition() const
{
	const TSharedRef<TPromise<FVector2D>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FVector2D>());

	AsyncTask(
		ENamedThreads::GameThread,
		[Promise]()
		{
			Promise->SetValue(FSlateApplication::Get().GetCursorPos());
		}
	);

	return TAsyncResult<FVector2D>(Promise->GetFuture(), nullptr, nullptr);
}

TAsyncResult<FModifierKeysState> FAsyncAutomationDriver::GetModifierKeys() const
{
	const TSharedRef<TPromise<FModifierKeysState>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FModifierKeysState>());

	AsyncTask(
		ENamedThreads::GameThread,
		[Promise]()
		{
			Promise->SetValue(FSlateApplication::Get().GetModifierKeys());
		}
	);

	return TAsyncResult<FModifierKeysState>(Promise->GetFuture(), nullptr, nullptr);
}

TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe> FAsyncAutomationDriver::FindElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	return FAsyncDriverElementFactory::Create(
		SharedThis(this),
		ElementLocator);
}

TSharedRef<IAsyncDriverElementCollection, ESPMode::ThreadSafe> FAsyncAutomationDriver::FindElements(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	return FAsyncDriverElementCollectionFactory::Create(
		SharedThis(this),
		ElementLocator);
}

TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe> FAsyncAutomationDriver::GetConfiguration() const
{
	return Configuration;
}

void FAsyncAutomationDriver::TrackPress(int32 KeyCode, int32 CharCode)
{
	if (KeyCode > 0)
	{
		PressedKeys.Add(KeyCode);
	}

	if (CharCode > 0)
	{
		PressedChars.Add(CharCode);
	}

	FKey Key = FInputKeyManager::Get().GetKeyFromCodes(KeyCode, CharCode);

	if (Key.IsModifierKey())
	{
		PressedModifiers.Add(Key);

		Application->SetFakeModifierKeys(FModifierKeysState(
			PressedModifiers.Contains(EKeys::LeftShift),
			PressedModifiers.Contains(EKeys::RightShift),
			PressedModifiers.Contains(EKeys::LeftControl),
			PressedModifiers.Contains(EKeys::RightControl),
			PressedModifiers.Contains(EKeys::LeftAlt),
			PressedModifiers.Contains(EKeys::RightAlt),
			PressedModifiers.Contains(EKeys::LeftCommand),
			PressedModifiers.Contains(EKeys::RightCommand),
			PressedModifiers.Contains(EKeys::CapsLock)
		));
	}
}

void FAsyncAutomationDriver::TrackPress(EMouseButtons::Type Button)
{
	PressedButtons.Add(Button);
}

void FAsyncAutomationDriver::TrackRelease(int32 KeyCode, int32 CharCode)
{
	if (KeyCode > 0)
	{
		PressedKeys.Remove(KeyCode);
	}

	if (CharCode > 0)
	{
		PressedChars.Remove(CharCode);
	}

	FKey Key = FInputKeyManager::Get().GetKeyFromCodes(KeyCode, CharCode);

	if (Key.IsModifierKey())
	{
		PressedModifiers.Remove(Key);

		Application->SetFakeModifierKeys(FModifierKeysState(
			PressedModifiers.Contains(EKeys::LeftShift),
			PressedModifiers.Contains(EKeys::RightShift),
			PressedModifiers.Contains(EKeys::LeftControl),
			PressedModifiers.Contains(EKeys::RightControl),
			PressedModifiers.Contains(EKeys::LeftAlt),
			PressedModifiers.Contains(EKeys::RightAlt),
			PressedModifiers.Contains(EKeys::LeftCommand),
			PressedModifiers.Contains(EKeys::RightCommand),
			PressedModifiers.Contains(EKeys::CapsLock)
		));
	}
}

void FAsyncAutomationDriver::TrackRelease(EMouseButtons::Type Button)
{
	PressedButtons.Remove(Button);
}

bool FAsyncAutomationDriver::IsPressed(int32 KeyCode, int32 CharCode) const
{
	return PressedKeys.Contains(KeyCode) || PressedChars.Contains(CharCode);
}

bool FAsyncAutomationDriver::IsPressed(EMouseButtons::Type Button) const
{
	return PressedButtons.Contains(Button);
}

int32 FAsyncAutomationDriver::ProcessCharacterForControlCodes(int32 CharCode) const
{
	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();

	if (ModifierKeys.IsControlDown())
	{
		const int32* ControlCode = CharactersToControlCodes.Find(CharCode);
		if (ControlCode != nullptr)
		{
			return *ControlCode;
		}
	}

	return CharCode;
}

void FAsyncAutomationDriver::Initialize()
{
#define AddControlCode(DisplayCharacter, ControlCharacter) CharactersToControlCodes.Add(DisplayCharacter, ControlCharacter);
	AddControlCode('@', 0);
	AddControlCode('A', 1);
	AddControlCode('B', 2);
	AddControlCode('C', 3);
	AddControlCode('D', 4);
	AddControlCode('E', 5);
	AddControlCode('F', 6);
	AddControlCode('G', 7);
	AddControlCode('H', 8);
	AddControlCode('I', 9);
	AddControlCode('J', 10);
	AddControlCode('K', 11);
	AddControlCode('L', 12);
	AddControlCode('M', 13);
	AddControlCode('N', 14);
	AddControlCode('O', 15);
	AddControlCode('P', 16);
	AddControlCode('Q', 17);
	AddControlCode('R', 18);
	AddControlCode('S', 19);
	AddControlCode('T', 20);
	AddControlCode('U', 21);
	AddControlCode('V', 22);
	AddControlCode('W', 23);
	AddControlCode('X', 24);
	AddControlCode('Y', 25);
	AddControlCode('Z', 26);

	AddControlCode('a', 1);
	AddControlCode('b', 2);
	AddControlCode('c', 3);
	AddControlCode('d', 4);
	AddControlCode('e', 5);
	AddControlCode('f', 6);
	AddControlCode('g', 7);
	AddControlCode('h', 8);
	AddControlCode('i', 9);
	AddControlCode('j', 10);
	AddControlCode('k', 11);
	AddControlCode('l', 12);
	AddControlCode('m', 13);
	AddControlCode('n', 14);
	AddControlCode('o', 15);
	AddControlCode('p', 16);
	AddControlCode('q', 17);
	AddControlCode('r', 18);
	AddControlCode('s', 19);
	AddControlCode('t', 20);
	AddControlCode('u', 21);
	AddControlCode('v', 22);
	AddControlCode('w', 23);
	AddControlCode('x', 24);
	AddControlCode('y', 25);
	AddControlCode('z', 26);

	AddControlCode('[', 27);
	AddControlCode('\\', 28);
	AddControlCode(']', 29);
	AddControlCode('^', 30);
	AddControlCode('_', 31);
#undef AddControlCode
}

TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> FAsyncAutomationDriverFactory::Create(
	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& AutomatedApplication)
{
	TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> Instance = MakeShareable(new FAsyncAutomationDriver(
		AutomatedApplication,
		MakeShareable(new FDriverConfiguration())));

	Instance->Initialize();
	return Instance;
}

TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> FAsyncAutomationDriverFactory::Create(
	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& AutomatedApplication,
	const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration)
{
	return MakeShareable(new FAsyncAutomationDriver(
		AutomatedApplication,
		Configuration));
}


FAutomationDriver::~FAutomationDriver()
{

}

bool FAutomationDriver::Wait(FTimespan Timespan)
{
	TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = CreateSequence();
	Sequence->Actions().Wait(MoveTemp(Timespan));
	return Sequence->Perform();
}

bool FAutomationDriver::Wait(const FDriverWaitDelegate& Delegate)
{
	TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = CreateSequence();
	Sequence->Actions().Wait(Delegate);
	return Sequence->Perform();
}

TSharedRef<IDriverSequence, ESPMode::ThreadSafe> FAutomationDriver::CreateSequence()
{
	return FDriverSequenceFactory::Create(FActionSequenceFactory::Create(
		SharedThis(this),
		AsyncDriver,
		Application));
}

FVector2D FAutomationDriver::GetCursorPosition() const
{
	const TSharedRef<TPromise<FVector2D>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FVector2D>());

	AsyncTask(
		ENamedThreads::GameThread,
		[Promise]()
		{
			Promise->SetValue(FSlateApplication::Get().GetCursorPos());
		}
	);

	return Promise->GetFuture().Get();
}

FModifierKeysState FAutomationDriver::GetModifierKeys() const
{
	const TSharedRef<TPromise<FModifierKeysState>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FModifierKeysState>());

	AsyncTask(
		ENamedThreads::GameThread,
		[Promise]()
		{
			Promise->SetValue(FSlateApplication::Get().GetModifierKeys());
		}
	);

	return Promise->GetFuture().Get();
}

TSharedRef<IDriverElement, ESPMode::ThreadSafe> FAutomationDriver::FindElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	return FDriverElementFactory::Create(
		SharedThis(this),
		ElementLocator);
}

TSharedRef<IDriverElementCollection, ESPMode::ThreadSafe> FAutomationDriver::FindElements(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	return FDriverElementCollectionFactory::Create(
		SharedThis(this),
		ElementLocator);
}

TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe> FAutomationDriver::GetConfiguration() const
{
	return AsyncDriver->GetConfiguration();
}

void FAutomationDriver::TrackPress(int32 KeyCode, int32 CharCode)
{
	AsyncDriver->TrackPress(KeyCode, CharCode);
}

void FAutomationDriver::TrackPress(EMouseButtons::Type Button)
{
	AsyncDriver->TrackPress(Button);
}

void FAutomationDriver::TrackRelease(int32 KeyCode, int32 CharCode)
{
	AsyncDriver->TrackRelease(KeyCode, CharCode);
}

void FAutomationDriver::TrackRelease(EMouseButtons::Type Button)
{
	AsyncDriver->TrackRelease(Button);
}

bool FAutomationDriver::IsPressed(int32 KeyCode, int32 CharCode) const
{
	return AsyncDriver->IsPressed(KeyCode, CharCode);
}

bool FAutomationDriver::IsPressed(EMouseButtons::Type Button) const
{
	return AsyncDriver->IsPressed(Button);
}

TSharedRef<FAutomationDriver, ESPMode::ThreadSafe> FAutomationDriverFactory::Create(
	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& AutomatedApplication)
{
	return MakeShareable(new FAutomationDriver(
		AutomatedApplication,
		FAsyncAutomationDriverFactory::Create(AutomatedApplication)));
}

TSharedRef<FAutomationDriver, ESPMode::ThreadSafe> FAutomationDriverFactory::Create(
	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& AutomatedApplication,
	const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration)
{
	return MakeShareable(new FAutomationDriver(
		AutomatedApplication,
		FAsyncAutomationDriverFactory::Create(AutomatedApplication, Configuration)));
}
