// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IAutomationDriver.h"
#include "WaitUntil.h"
#include "Misc/Timespan.h"
#include "InputCoreTypes.h"

class FAutomatedApplication;
class IElementLocator;
class IAsyncDriverSequence;
class IAsyncActionSequence;
class IAsyncDriverElement;
class IAsyncDriverElementCollection;
class FDriverConfiguration;
class FModifierKeysState;

class FAsyncAutomationDriverFactory;

class FAsyncAutomationDriver
	: public IAsyncAutomationDriver
	, public TSharedFromThis<FAsyncAutomationDriver, ESPMode::ThreadSafe>
{
public:

	virtual ~FAsyncAutomationDriver();

	virtual TAsyncResult<bool> Wait(FTimespan Timespan) override;
	virtual TAsyncResult<bool> Wait(const FDriverWaitDelegate& Delegate) override;

	virtual TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> CreateSequence() override;

	virtual TAsyncResult<FVector2D> GetCursorPosition() const override;

	virtual TAsyncResult<FModifierKeysState> GetModifierKeys() const override;

	virtual TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe> FindElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override;

	virtual TSharedRef<IAsyncDriverElementCollection, ESPMode::ThreadSafe> FindElements(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override;

	virtual TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe> GetConfiguration() const override;

	/**
	 * Saves a shared pointer to the action sequence object to ensure it remains in memory until execution is complete.
	 * @param Sequence shared pointer to the action sequence.
	 * @return true if the pointer has been saved successfully; false if another pinned sequence is currently being executed.
	 */
	bool PinActionSequence(const TSharedPtr<IAsyncActionSequence, ESPMode::ThreadSafe>& Sequence);
	
	/**
	 * Resets an internally stored shared pointer to the previously pinned action sequence object so that it can be deleted.
	 * @param Sequence shared pointer to the action sequence.
	 * @return true if the pointer has been reset successfully; false if the provided sequence hasn't been pinned previously or if it is still executing
	 */
	bool UnpinActionSequence(const TSharedPtr<IAsyncActionSequence, ESPMode::ThreadSafe>& Sequence);

public:

	void TrackPress(int32 KeyCode, int32 CharCode);
	void TrackPress(EMouseButtons::Type Button);

	void TrackRelease(int32 KeyCode, int32 CharCode);
	void TrackRelease(EMouseButtons::Type Button);

	bool IsPressed(int32 KeyCode, int32 CharCode) const;
	bool IsPressed(EMouseButtons::Type Button) const;

	int32 ProcessCharacterForControlCodes(int32 CharCode) const;

private:

	FAsyncAutomationDriver(
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& InApplication,
		const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& InConfiguration)
		: Application(InApplication)
		, Configuration(InConfiguration)
	{ }

	void Initialize();

private:

	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe> Application;
	const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe> Configuration;
	TSharedPtr<IAsyncActionSequence, ESPMode::ThreadSafe> PinnedSequence;

	TSet<FKey> PressedModifiers;
	TSet<int32> PressedKeys;
	TSet<int32> PressedChars;
	TSet<EMouseButtons::Type> PressedButtons;
	TMap<int32, int32> CharactersToControlCodes;

	friend FAsyncAutomationDriverFactory;
};

class FAsyncAutomationDriverFactory
{
public:

	static TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& AutomatedApplication);

	static TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& AutomatedApplication,
		const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration);
};


class FAutomationDriverFactory;

class FAutomationDriver
	: public IAutomationDriver
	, public TSharedFromThis<FAutomationDriver, ESPMode::ThreadSafe>
{
public:

	virtual ~FAutomationDriver();

	virtual bool Wait(FTimespan Timespan) override;
	virtual bool Wait(const FDriverWaitDelegate& Delegate) override;

	virtual TSharedRef<IDriverSequence, ESPMode::ThreadSafe> CreateSequence() override;

	virtual FVector2D GetCursorPosition() const override;

	virtual FModifierKeysState GetModifierKeys() const override;

	virtual TSharedRef<IDriverElement, ESPMode::ThreadSafe> FindElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override;

	virtual TSharedRef<IDriverElementCollection, ESPMode::ThreadSafe> FindElements(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override;

	virtual TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe> GetConfiguration() const override;

public:

	void TrackPress(int32 KeyCode, int32 CharCode);
	void TrackPress(EMouseButtons::Type Button);

	void TrackRelease(int32 KeyCode, int32 CharCode);
	void TrackRelease(EMouseButtons::Type Button);

	bool IsPressed(int32 KeyCode, int32 CharCode) const;
	bool IsPressed(EMouseButtons::Type Button) const;

private:

	FAutomationDriver(
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& InApplication,
		const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& InAsyncDriver)
		: Application(InApplication)
		, AsyncDriver(InAsyncDriver)
	{ }

private:

	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe> Application;
	const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> AsyncDriver;

	friend FAutomationDriverFactory;
};

class FAutomationDriverFactory
{
public:

	static TSharedRef<FAutomationDriver, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& AutomatedApplication);

	static TSharedRef<FAutomationDriver, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& AutomatedApplication,
		const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration);
};
