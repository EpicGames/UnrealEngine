// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationDriverLogging.h"
#include "AutomationDriverCommon.h"
#include "IApplicationElement.h"

DEFINE_LOG_CATEGORY(LogAutomationDriver);

void FAutomationDriverLogging::TooManyElementsFound(const TArray<TSharedRef<IApplicationElement>>& Elements)
{
	UE_LOGF(LogAutomationDriver, Error, "Multiple elements found when 1 was expected\nExpected 1\n   Found %d", Elements.Num());

	for (int32 Index = 0; Index < Elements.Num(); Index++)
	{
		const TSharedRef<IApplicationElement>& Element = Elements[Index];
		UE_LOGF(LogAutomationDriver, Error, "    [%d] -> %ls", Index, *Element->ToDebugString());
	}
}

void FAutomationDriverLogging::CannotFindElement(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	UE_LOGF(LogAutomationDriver, Error, "Failed to locate element");

	if (ElementLocator.IsValid())
	{
		UE_LOGF(LogAutomationDriver, Error, "    %ls", *ElementLocator->ToDebugString());
	}
}

void FAutomationDriverLogging::ElementNotVisible(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	UE_LOGF(LogAutomationDriver, Error, "Failed to locate visible element");

	if (ElementLocator.IsValid())
	{
		UE_LOGF(LogAutomationDriver, Error, "    Element found but not visible: %ls", *ElementLocator->ToDebugString());
	}
	else
	{
		UE_LOGF(LogAutomationDriver, Error, "    Element found but not visible");
	}
}

void FAutomationDriverLogging::ElementNotInteractable(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	UE_LOGF(LogAutomationDriver, Error, "Failed to locate interactable element");

	if (ElementLocator.IsValid())
	{
		UE_LOGF(LogAutomationDriver, Error, "    Element found but not interactable: %ls", *ElementLocator->ToDebugString());
	}
	else
	{
		UE_LOGF(LogAutomationDriver, Error, "    Element found but not interactable");
	}
}

void FAutomationDriverLogging::ElementHasNoWindow(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	UE_LOGF(LogAutomationDriver, Error, "Failed to locate window hosting element");

	if (ElementLocator.IsValid())
	{
		UE_LOGF(LogAutomationDriver, Error, "    Element found but no window is associated with it: %ls", *ElementLocator->ToDebugString());
	}
	else
	{
		UE_LOGF(LogAutomationDriver, Error, "    Element found but no window is associated with it");
	}
}

void FAutomationDriverLogging::CannotClickUnhoveredElement(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	UE_LOGF(LogAutomationDriver, Error, "Failed to click element");

	if (ElementLocator.IsValid())
	{
		UE_LOGF(LogAutomationDriver, Error, "    Element found but not located under the cursor: %ls", *ElementLocator->ToDebugString());
	}
	else
	{
		UE_LOGF(LogAutomationDriver, Error, "    Element found but not located under the cursor");
	}
}

void FAutomationDriverLogging::CannotExecuteMultipleActionSequencesInParallel()
{
	UE_LOGF(LogAutomationDriver, Error, "Parallel execution of multiple action sequences is not supported");
}

void FAutomationDriverLogging::CannotUnpinActionSequenceIfNotPinned()
{
	UE_LOGF(LogAutomationDriver, Error, "Cannot unpin action sequence if it's not pinned");
}

void FAutomationDriverLogging::CannotUnpinActionSequenceIfExecuting()
{
	UE_LOGF(LogAutomationDriver, Error, "Cannot unpin action sequence if it's still executing");
}

