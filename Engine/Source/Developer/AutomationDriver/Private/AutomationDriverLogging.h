// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IApplicationElement;
class IElementLocator;

DECLARE_LOG_CATEGORY_EXTERN(LogAutomationDriver, Error, All);

class FAutomationDriverLogging
{
public:

	static void TooManyElementsFound(const TArray<TSharedRef<IApplicationElement>>& Elements);

	static void CannotFindElement(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator);

	static void ElementNotVisible(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator);

	static void ElementNotInteractable(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator);

	static void ElementHasNoWindow(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator);

	static void CannotClickUnhoveredElement(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator);
	
	static void CannotExecuteMultipleActionSequencesInParallel();

	static void CannotUnpinActionSequenceIfNotPinned();

	static void CannotUnpinActionSequenceIfExecuting();
};
