// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAutomationDriverModule.h"
#include "PassThroughMessageHandler.h"
#include "AutomatedApplication.h"
#include "AutomationDriver.h"
#include "Framework/Application/SlateApplication.h"

class FAutomationDriverModule
	: public IAutomationDriverModule
{
public:

	virtual void StartupModule() override
	{

	}

	virtual void ShutdownModule() override
	{

	}

	virtual TSharedRef<IAutomationDriver, ESPMode::ThreadSafe> CreateDriver() const override
	{
		return FAutomationDriverFactory::Create(AutomatedApplication.ToSharedRef());
	}

	virtual TSharedRef<IAutomationDriver, ESPMode::ThreadSafe> CreateDriver(const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration) const override
	{
		return FAutomationDriverFactory::Create(AutomatedApplication.ToSharedRef(), Configuration);
	}

	virtual TSharedRef<IAsyncAutomationDriver, ESPMode::ThreadSafe> CreateAsyncDriver() const override
	{
		return FAsyncAutomationDriverFactory::Create(AutomatedApplication.ToSharedRef());
	}

	virtual TSharedRef<IAsyncAutomationDriver, ESPMode::ThreadSafe> CreateAsyncDriver(const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration) const override
	{
		return FAsyncAutomationDriverFactory::Create(AutomatedApplication.ToSharedRef(), Configuration);
	}

	virtual bool IsEnabled() const override
	{
		return AutomatedApplication.IsValid();
	}

	virtual void Enable() override
	{
		if (IsEnabled())
		{
			return;
		}

		RealApplication = FSlateApplication::Get().GetPlatformApplication();
		RealMessageHandler = RealApplication->GetMessageHandler();

		AutomatedApplication = FAutomatedApplicationFactory::Create(
			RealApplication.ToSharedRef(),
			FPassThroughMessageHandlerFactoryFactory::Create());

		if (AutomatedApplication.IsValid())
		{
			FSlateApplication::Get().SetPlatformApplication(AutomatedApplication.ToSharedRef());
			AutomatedApplication->AllowPlatformMessageHandling();
		}
	}

	virtual void Disable() override
	{
		if (!IsEnabled())
		{
			return;
		}

		AutomatedApplication->DisablePlatformMessageHandling();

#if WITH_ACCESSIBILITY
		// Unregister primary user on the accessible handler before swapping back the original application
		FGenericAccessibleUserRegistry& UserRegistry = AutomatedApplication->GetAccessibleMessageHandler()->GetAccessibleUserRegistry();
		UserRegistry.UnregisterUser(FGenericAccessibleUserRegistry::GetPrimaryUserIndex());
#endif

		FSlateApplication::Get().SetPlatformApplication(RealApplication.ToSharedRef());
		RealApplication->SetMessageHandler(RealMessageHandler.ToSharedRef());

		AutomatedApplication.Reset();
		// Note that here we have to return the platform cursor that was overriden by FAutomatedCursor while enabling the module.
		// Sometimes it is not being replaced with the real platform cursor, so we force it.
		// It is needed to prevent possible issues that we are sending mouse events to the cursor that is outdated while executing the next tests from the list of AutomationDriver tests.
		FSlateApplication::Get().UsePlatformCursorForCursorUser(true);

		RealApplication.Reset();
		RealMessageHandler.Reset();
	}

private:

	TSharedPtr<FAutomatedApplication> AutomatedApplication;
	TSharedPtr<GenericApplication> RealApplication;
	TSharedPtr<FGenericApplicationMessageHandler> RealMessageHandler;
};

IMPLEMENT_MODULE(FAutomationDriverModule, AutomationDriver);
