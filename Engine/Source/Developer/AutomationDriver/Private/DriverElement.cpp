// Copyright Epic Games, Inc. All Rights Reserved.

#include "DriverElement.h"
#include "IDriverElement.h"
#include "IDriverSequence.h"
#include "IElementLocator.h"
#include "IApplicationElement.h"

#include "AutomationDriver.h"

#include "Async/Async.h"
#include "Async/AsyncResult.h"
#include "InputCoreTypes.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

class FDriverElementExtensions
{
private:

	static TSharedPtr<IApplicationElement> LocateSingleElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		if (Elements.Num() > 1)
		{
			return nullptr;
		}
		else if (Elements.Num() == 0)
		{
			return nullptr;
		}

		return Elements[0];
	}

public:


	static bool CanFocus(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->CanFocus();
	}

	static bool IsFocused(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->IsFocused();
	}

	static bool IsFocused(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, uint32 UserIndex)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->IsFocused(UserIndex);
	}

	static bool HasFocusedDescendants(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->HasFocusedDescendants();
	}

	static bool HasFocusedDescendants(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, uint32 UserIndex)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->HasFocusedDescendants(UserIndex);
	}

	static bool Exists(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		return LocateSingleElement(ElementLocator).IsValid();
	}

	static bool IsChecked(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->IsChecked();
	}

	static bool IsInteractable(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->IsInteractable();
	}

	static bool IsHovered(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->IsHovered();
	}

	static bool IsVisible(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->IsVisible();
	}

	static bool IsScrollable(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->IsScrollable();
	}

	static bool IsScrolledToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->IsScrolledToBeginning();
	}

	static bool IsScrolledToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return false;
		}

		return Element->IsScrolledToEnd();
	}

	static FVector2D GetAbsolutePosition(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return FVector2D::ZeroVector;
		}

		return Element->GetAbsolutePosition();
	}

	static FVector2D GetSize(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return FVector2D::ZeroVector;
		}

		return Element->GetSize();
	}

	static FText GetText(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		const TSharedPtr<IApplicationElement> Element = LocateSingleElement(ElementLocator);

		if (!Element.IsValid())
		{
			return FText::GetEmpty();
		}

		return Element->GetText();
	}
};

class FAsyncDriverElementCollection
	: public IAsyncDriverElementCollection
	, public TSharedFromThis<FAsyncDriverElementCollection, ESPMode::ThreadSafe>
{
public:

	virtual ~FAsyncDriverElementCollection()
	{ }

	virtual TAsyncResult<TArray<TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe>>> GetElements() override
	{
		const TSharedRef<TPromise<TArray<TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe>>>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<TArray<TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe>>>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;
		const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> Driver = AsyncDriver;

		AsyncTask(
			ENamedThreads::GameThread,
			[Driver, Locator, Promise]()
			{
				TArray<TSharedRef<IApplicationElement>> AppElements;
				Locator->Locate(AppElements);

				TArray<TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe>> DriverElements;

				for (int32 Index = 0; Index < AppElements.Num(); ++Index)
				{
					DriverElements.Add(FAsyncDriverElementFactory::Create(
						Driver,
						AppElements[Index]->CreateLocator()));
				}

				Promise->SetValue(DriverElements);
			}
		);

		return TAsyncResult<TArray<TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe>>>(Promise->GetFuture(), nullptr, nullptr);
	}

private:

	FAsyncDriverElementCollection(
		const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& InAsyncDriver,
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& InElementLocator)
		: AsyncDriver(InAsyncDriver)
		, ElementLocator(InElementLocator)
	{ }


private:

	const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> AsyncDriver;
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe> ElementLocator;

	friend FAsyncDriverElementCollectionFactory;
};

TSharedRef<IAsyncDriverElementCollection, ESPMode::ThreadSafe> FAsyncDriverElementCollectionFactory::Create(
	const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver,
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	return MakeShareable(new FAsyncDriverElementCollection(
		AsyncDriver,
		ElementLocator));
}

class FAsyncDriverElement
	: public IAsyncDriverElement
	, public TSharedFromThis<FAsyncDriverElement, ESPMode::ThreadSafe>
{
public:

	virtual ~FAsyncDriverElement()
	{ }

	virtual TAsyncResult<bool> Hover() override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().MoveToElement(SharedThis(this));
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Click(EMouseButtons::Type MouseButton) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Click(SharedThis(this), MouseButton);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Click() override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Click(SharedThis(this), EMouseButtons::Left);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> DoubleClick(EMouseButtons::Type MouseButton) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().DoubleClick(SharedThis(this), MouseButton);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> DoubleClick() override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().DoubleClick(SharedThis(this), EMouseButtons::Left);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ScrollBy(float Delta) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ScrollBy(SharedThis(this), Delta);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ScrollToBeginning() override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ScrollToBeginning(SharedThis(this));
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ScrollToBeginning(float Amount) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ScrollToBeginning(SharedThis(this), Amount);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ScrollToBeginningUntil(SharedThis(this), DesiredElementLocator);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ScrollToEnd() override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ScrollToEnd(SharedThis(this));
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ScrollToEnd(float Amount) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ScrollToEnd(SharedThis(this), Amount);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ScrollToEndUntil(SharedThis(this), DesiredElementLocator);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Type(const TCHAR* Text) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), FString(Text));
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Type(FString Text) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), MoveTemp(Text));
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Type(FKey Key) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), Key);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Type(TCHAR Character) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), Character);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Type(const TArray<FKey>& Keys) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), Keys);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> TypeChord(FKey Key1, FKey Key2) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().TypeChord(SharedThis(this), Key1, Key2);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> TypeChord(FKey Key1, TCHAR Character) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().TypeChord(SharedThis(this), Key1, Character);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> TypeChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().TypeChord(SharedThis(this), Key1, Key2, Key3);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> TypeChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().TypeChord(SharedThis(this), Key1, Key2, Character);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Press(TCHAR Character) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Press(SharedThis(this), Character);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Press(FKey Key) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Press(SharedThis(this), Key);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Press(EMouseButtons::Type MouseButton) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Press(SharedThis(this), MouseButton);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> PressChord(FKey Key1, FKey Key2) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().PressChord(SharedThis(this), Key1, Key2);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> PressChord(FKey Key1, TCHAR Character) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().PressChord(SharedThis(this), Key1, Character);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> PressChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().PressChord(SharedThis(this), Key1, Key2, Key3);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> PressChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().PressChord(SharedThis(this), Key1, Key2, Character);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Release(TCHAR Character) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Release(SharedThis(this), Character);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Release(FKey Key) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Release(SharedThis(this), Key);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Release(EMouseButtons::Type MouseButton) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Release(SharedThis(this), MouseButton);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, FKey Key2) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ReleaseChord(SharedThis(this), Key1, Key2);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, TCHAR Character) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ReleaseChord(SharedThis(this), Key1, Character);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ReleaseChord(SharedThis(this), Key1, Key2, Key3);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().ReleaseChord(SharedThis(this), Key1, Key2, Character);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Focus() override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Focus(SharedThis(this));
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> Focus(uint32 UserFocus) override
	{
		TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Sequence = AsyncDriver->CreateSequence();
		Sequence->Actions().Focus(SharedThis(this), UserFocus);
		return Sequence->Perform();
	}

	virtual TAsyncResult<bool> CanFocus() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::CanFocus(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> IsFocused() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsFocused(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> IsFocused(uint32 UserIndex) const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, UserIndex, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsFocused(Locator, UserIndex));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> HasFocusedDescendants() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::HasFocusedDescendants(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> HasFocusedDescendants(uint32 UserIndex) const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, UserIndex, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::HasFocusedDescendants(Locator, UserIndex));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> Exists() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::Exists(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> IsVisible() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsVisible(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> IsChecked() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsChecked(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> IsInteractable() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsInteractable(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> IsScrollable() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsScrollable(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> IsScrolledToBeginning() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsScrolledToBeginning(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> IsScrolledToEnd() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsScrolledToEnd(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<bool> IsHovered() const override
	{
		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsHovered(Locator));
			}
		);

		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<FVector2D> GetAbsolutePosition() const override
	{
		const TSharedRef<TPromise<FVector2D>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FVector2D>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::GetAbsolutePosition(Locator));
			}
		);

		return TAsyncResult<FVector2D>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<FVector2D> GetSize() const override
	{
		const TSharedRef<TPromise<FVector2D>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FVector2D>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::GetSize(Locator));
			}
		);

		return TAsyncResult<FVector2D>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual TAsyncResult<FText> GetText() const override
	{
		const TSharedRef<TPromise<FText>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FText>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::GetText(Locator));
			}
		);

		return TAsyncResult<FText>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const override
	{
		ElementLocator->Locate(OutElements);
	}

	virtual FString ToDebugString() const override
	{
		return ElementLocator->ToDebugString();
	}

private:

	FAsyncDriverElement(
		const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& InAsyncDriver,
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& InElementLocator)
		: AsyncDriver(InAsyncDriver)
		, ElementLocator(InElementLocator)
	{ }


private:

	const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> AsyncDriver;
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe> ElementLocator;

	friend FAsyncDriverElementFactory;
};

TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe> FAsyncDriverElementFactory::Create(
	const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver,
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	return MakeShareable(new FAsyncDriverElement(
		AsyncDriver,
		ElementLocator));
}

class FEmptyAsyncDriverElement
	: public IAsyncDriverElement
	, public TSharedFromThis<FEmptyAsyncDriverElement, ESPMode::ThreadSafe>
{
public:

	virtual ~FEmptyAsyncDriverElement()
	{ }

	virtual TAsyncResult<bool> Hover() override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Click(EMouseButtons::Type MouseButton) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Click() override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> DoubleClick(EMouseButtons::Type MouseButton) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> DoubleClick() override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ScrollBy(float Delta) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ScrollToBeginning() override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ScrollToBeginning(float Amount) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ScrollToEnd() override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ScrollToEnd(float Amount) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Type(const TCHAR* Text) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Type(FString Text) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Type(FKey Key) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Type(TCHAR Character) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Type(const TArray<FKey>& Keys) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> TypeChord(FKey Key1, FKey Key2) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> TypeChord(FKey Key1, TCHAR Character) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> TypeChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> TypeChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Press(TCHAR Character) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Press(FKey Key) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Press(EMouseButtons::Type MouseButton) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> PressChord(FKey Key1, FKey Key2) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> PressChord(FKey Key1, TCHAR Character) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> PressChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> PressChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Release(TCHAR Character) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Release(FKey Key) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Release(EMouseButtons::Type MouseButton) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, FKey Key2) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, TCHAR Character) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Focus() override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Focus(uint32 UserFocus) override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> CanFocus() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> IsFocused() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> IsFocused(uint32 UserIndex) const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> HasFocusedDescendants() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> HasFocusedDescendants(uint32 UserIndex) const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> Exists() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> IsVisible() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> IsChecked() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> IsInteractable() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> IsScrollable() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> IsScrolledToBeginning() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> IsScrolledToEnd() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<bool> IsHovered() const override
	{
		return TAsyncResult<bool>(false);
	}

	virtual TAsyncResult<FVector2D> GetAbsolutePosition() const override
	{
		return TAsyncResult<FVector2D>(FVector2D(0, 0));
	}

	virtual TAsyncResult<FVector2D> GetSize() const override
	{
		return TAsyncResult<FVector2D>(FVector2D(0, 0));
	}

	virtual TAsyncResult<FText> GetText() const override
	{
		return TAsyncResult<FText>(FText::GetEmpty());
	}

	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const override
	{
		
	}

	virtual FString ToDebugString() const override
	{
		return TEXT("Empty Driver Element");
	}

private:

	FEmptyAsyncDriverElement()
	{ }


private:

	friend FEmptyAsyncDriverElementFactory;
};

TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe> FEmptyAsyncDriverElementFactory::Create()
{
	return MakeShareable(new FEmptyAsyncDriverElement());
}

class FDriverElementCollection
	: public IDriverElementCollection
	, public TSharedFromThis<FDriverElementCollection, ESPMode::ThreadSafe>
{
public:

	virtual ~FDriverElementCollection()
	{ }

	virtual TArray<TSharedRef<IDriverElement, ESPMode::ThreadSafe>> GetElements() override
	{
		if (IsInGameThread())
		{
			return InternalGetElement();
		}

		const TSharedRef<FDriverElementCollection, ESPMode::ThreadSafe> LocalThis = SharedThis(this);
		const TSharedRef<TPromise<TArray<TSharedRef<IDriverElement, ESPMode::ThreadSafe>>>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<TArray<TSharedRef<IDriverElement, ESPMode::ThreadSafe>>>());

		AsyncTask(
			ENamedThreads::GameThread,
			[LocalThis, Promise]()
			{
				Promise->SetValue(LocalThis->InternalGetElement());
			}
		);

		return Promise->GetFuture().Get();
	}

private:

	TArray<TSharedRef<IDriverElement, ESPMode::ThreadSafe>> InternalGetElement()
	{
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> AppElements;
		ElementLocator->Locate(AppElements);

		TArray<TSharedRef<IDriverElement, ESPMode::ThreadSafe>> DriverElements;

		for (int32 Index = 0; Index < AppElements.Num(); ++Index)
		{
			DriverElements.Add(FDriverElementFactory::Create(
				Driver,
				AppElements[Index]->CreateLocator()));
		}

		return DriverElements;
	}

private:

	FDriverElementCollection(
		const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe>& InDriver,
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& InElementLocator)
		: Driver(InDriver)
		, ElementLocator(InElementLocator)
	{ }

private:

	const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe> Driver;
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe> ElementLocator;

	friend FDriverElementCollectionFactory;
};

TSharedRef<IDriverElementCollection, ESPMode::ThreadSafe> FDriverElementCollectionFactory::Create(
	const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe>& Driver,
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	return MakeShareable(new FDriverElementCollection(
		Driver,
		ElementLocator));
}

class FDriverElement
	: public IDriverElement
	, public TSharedFromThis<FDriverElement, ESPMode::ThreadSafe>
{
public:

	virtual ~FDriverElement()
	{ }

	virtual bool Hover() override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().MoveToElement(SharedThis(this));
		return Sequence->Perform();
	}

	virtual bool Click(EMouseButtons::Type MouseButton) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Click(SharedThis(this), MouseButton);
		return Sequence->Perform();
	}

	virtual bool Click() override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Click(SharedThis(this), EMouseButtons::Left);
		return Sequence->Perform();
	}

	virtual bool DoubleClick() override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().DoubleClick(SharedThis(this), EMouseButtons::Left);
		return Sequence->Perform();
	}

	virtual bool DoubleClick(EMouseButtons::Type MouseButton) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().DoubleClick(SharedThis(this), MouseButton);
		return Sequence->Perform();
	}

	virtual bool ScrollBy(float Delta) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ScrollBy(SharedThis(this), Delta);
		return Sequence->Perform();
	}

	virtual bool ScrollToBeginning() override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ScrollToBeginning(SharedThis(this));
		return Sequence->Perform();
	}

	virtual bool ScrollToBeginning(float Amount) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ScrollToBeginning(SharedThis(this), Amount);
		return Sequence->Perform();
	}

	virtual bool ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ScrollToBeginningUntil(SharedThis(this), DesiredElementLocator);
		return Sequence->Perform();
	}

	virtual bool ScrollToEnd() override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ScrollToEnd(SharedThis(this));
		return Sequence->Perform();
	}

	virtual bool ScrollToEnd(float Amount) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ScrollToEnd(SharedThis(this), Amount);
		return Sequence->Perform();
	}

	virtual bool ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ScrollToEndUntil(SharedThis(this), DesiredElementLocator);
		return Sequence->Perform();
	}

	virtual bool Type(const TCHAR* Text) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), FString(Text));
		return Sequence->Perform();
	}

	virtual bool Type(FString Text) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), MoveTemp(Text));
		return Sequence->Perform();
	}

	virtual bool Type(FKey Key) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), Key);
		return Sequence->Perform();
	}

	virtual bool Type(TCHAR Character) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), Character);
		return Sequence->Perform();
	}

	virtual bool Type(const TArray<FKey>& Keys) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Type(SharedThis(this), Keys);
		return Sequence->Perform();
	}

	virtual bool TypeChord(FKey Key1, FKey Key2) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().TypeChord(SharedThis(this), Key1, Key2);
		return Sequence->Perform();
	}

	virtual bool TypeChord(FKey Key1, TCHAR Character) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().TypeChord(SharedThis(this), Key1, Character);
		return Sequence->Perform();
	}

	virtual bool TypeChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().TypeChord(SharedThis(this), Key1, Key2, Key3);
		return Sequence->Perform();
	}

	virtual bool TypeChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().TypeChord(SharedThis(this), Key1, Key2, Character);
		return Sequence->Perform();
	}

	virtual bool Press(TCHAR Character) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Press(SharedThis(this), Character);
		return Sequence->Perform();
	}

	virtual bool Press(FKey Key) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Press(SharedThis(this), Key);
		return Sequence->Perform();
	}

	virtual bool Press(EMouseButtons::Type MouseButton) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Press(SharedThis(this), MouseButton);
		return Sequence->Perform();
	}

	virtual bool PressChord(FKey Key1, FKey Key2) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().PressChord(SharedThis(this), Key1, Key2);
		return Sequence->Perform();
	}

	virtual bool PressChord(FKey Key1, TCHAR Character) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().PressChord(SharedThis(this), Key1, Character);
		return Sequence->Perform();
	}

	virtual bool PressChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().PressChord(SharedThis(this), Key1, Key2, Key3);
		return Sequence->Perform();
	}

	virtual bool PressChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().PressChord(SharedThis(this), Key1, Key2, Character);
		return Sequence->Perform();
	}

	virtual bool Release(TCHAR Character) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Release(SharedThis(this), Character);
		return Sequence->Perform();
	}

	virtual bool Release(FKey Key) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Release(SharedThis(this), Key);
		return Sequence->Perform();
	}

	virtual bool Release(EMouseButtons::Type MouseButton) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Release(SharedThis(this), MouseButton);
		return Sequence->Perform();
	}

	virtual bool ReleaseChord(FKey Key1, FKey Key2) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ReleaseChord(SharedThis(this), Key1, Key2);
		return Sequence->Perform();
	}

	virtual bool ReleaseChord(FKey Key1, TCHAR Character) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ReleaseChord(SharedThis(this), Key1, Character);
		return Sequence->Perform();
	}

	virtual bool ReleaseChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ReleaseChord(SharedThis(this), Key1, Key2, Key3);
		return Sequence->Perform();
	}

	virtual bool ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().ReleaseChord(SharedThis(this), Key1, Key2, Character);
		return Sequence->Perform();
	}

	virtual bool Focus() override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Focus(SharedThis(this));
		return Sequence->Perform();
	}

	virtual bool Focus(uint32 UserIndex) override
	{
		TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Sequence = Driver->CreateSequence();
		Sequence->Actions().Focus(SharedThis(this), UserIndex);
		return Sequence->Perform();
	}

	virtual bool CanFocus() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::CanFocus(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::CanFocus(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool IsFocused() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::IsFocused(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsFocused(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool IsFocused(uint32 UserIndex) const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::IsFocused(ElementLocator, UserIndex);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, UserIndex, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsFocused(Locator, UserIndex));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool HasFocusedDescendants() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::HasFocusedDescendants(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::HasFocusedDescendants(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool HasFocusedDescendants(uint32 UserIndex) const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::HasFocusedDescendants(ElementLocator, UserIndex);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, UserIndex, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::HasFocusedDescendants(Locator, UserIndex));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool Exists() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::Exists(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::Exists(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool IsVisible() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::IsVisible(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsVisible(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool IsChecked() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::IsChecked(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsChecked(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool IsInteractable() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::IsInteractable(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsInteractable(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool IsScrollable() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::IsScrollable(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsScrollable(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool IsScrolledToBeginning() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::IsScrolledToBeginning(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsScrolledToBeginning(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool IsScrolledToEnd() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::IsScrolledToEnd(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsScrolledToEnd(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual bool IsHovered() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::IsHovered(ElementLocator);
		}

		const TSharedRef<TPromise<bool>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<bool>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::IsHovered(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual FVector2D GetAbsolutePosition() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::GetAbsolutePosition(ElementLocator);
		}

		const TSharedRef<TPromise<FVector2D>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FVector2D>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::GetAbsolutePosition(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual FVector2D GetSize() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::GetSize(ElementLocator);
		}

		const TSharedRef<TPromise<FVector2D>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FVector2D>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::GetSize(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual FText GetText() const override
	{
		if (IsInGameThread())
		{
			return FDriverElementExtensions::GetText(ElementLocator);
		}

		const TSharedRef<TPromise<FText>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<FText>());
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe> Locator = ElementLocator;

		AsyncTask(
			ENamedThreads::GameThread,
			[Locator, Promise]()
			{
				Promise->SetValue(FDriverElementExtensions::GetText(Locator));
			}
		);

		return Promise->GetFuture().Get();
	}

	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const override
	{
		ElementLocator->Locate(OutElements);
	}

	virtual FString ToDebugString() const override
	{
		return ElementLocator->ToDebugString();
	}

private:

	FDriverElement(
		const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe>& InDriver,
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& InElementLocator)
		: Driver(InDriver)
		, ElementLocator(InElementLocator)
	{ }

private:

	const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe> Driver;
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe> ElementLocator;

	friend FDriverElementFactory;
};

TSharedRef<IDriverElement, ESPMode::ThreadSafe> FDriverElementFactory::Create(
	const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe>& Driver,
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
{
	return MakeShareable(new FDriverElement(
		Driver,
		ElementLocator));
}

class FEmptyDriverElement
	: public IDriverElement
	, public TSharedFromThis<FEmptyDriverElement, ESPMode::ThreadSafe>
{
public:

	virtual ~FEmptyDriverElement()
	{ }

	virtual bool Hover() override
	{
		return false;
	}

	virtual bool Click(EMouseButtons::Type MouseButton) override
	{
		return false;
	}

	virtual bool Click() override
	{
		return false;
	}

	virtual bool DoubleClick() override
	{
		return false;
	}

	virtual bool DoubleClick(EMouseButtons::Type MouseButton) override
	{
		return false;
	}

	virtual bool ScrollBy(float Delta) override
	{
		return false;
	}

	virtual bool ScrollToBeginning() override
	{
		return false;
	}

	virtual bool ScrollToBeginning(float Amount) override
	{
		return false;
	}

	virtual bool ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) override
	{
		return false;
	}

	virtual bool ScrollToEnd() override
	{
		return false;
	}

	virtual bool ScrollToEnd(float Amount) override
	{
		return false;
	}

	virtual bool ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) override
	{
		return false;
	}

	virtual bool Type(const TCHAR* Text) override
	{
		return false;
	}

	virtual bool Type(FString Text) override
	{
		return false;
	}

	virtual bool Type(FKey Key) override
	{
		return false;
	}

	virtual bool Type(TCHAR Character) override
	{
		return false;
	}

	virtual bool Type(const TArray<FKey>& Keys) override
	{
		return false;
	}

	virtual bool TypeChord(FKey Key1, FKey Key2) override
	{
		return false;
	}

	virtual bool TypeChord(FKey Key1, TCHAR Character) override
	{
		return false;
	}

	virtual bool TypeChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		return false;
	}

	virtual bool TypeChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		return false;
	}

	virtual bool Press(TCHAR Character) override
	{
		return false;
	}

	virtual bool Press(FKey Key) override
	{
		return false;
	}

	virtual bool Press(EMouseButtons::Type MouseButton) override
	{
		return false;
	}

	virtual bool PressChord(FKey Key1, FKey Key2) override
	{
		return false;
	}

	virtual bool PressChord(FKey Key1, TCHAR Character) override
	{
		return false;
	}

	virtual bool PressChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		return false;
	}

	virtual bool PressChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		return false;
	}

	virtual bool Release(TCHAR Character) override
	{
		return false;
	}

	virtual bool Release(FKey Key) override
	{
		return false;
	}

	virtual bool Release(EMouseButtons::Type MouseButton) override
	{
		return false;
	}

	virtual bool ReleaseChord(FKey Key1, FKey Key2) override
	{
		return false;
	}

	virtual bool ReleaseChord(FKey Key1, TCHAR Character) override
	{
		return false;
	}

	virtual bool ReleaseChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		return false;
	}

	virtual bool ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		return false;
	}

	virtual bool Focus() override
	{
		return false;
	}

	virtual bool Focus(uint32 UserIndex) override
	{
		return false;
	}

	virtual bool CanFocus() const override
	{
		return false;
	}

	virtual bool IsFocused() const override
	{
		return false;
	}

	virtual bool IsFocused(uint32 UserIndex) const override
	{
		return false;
	}

	virtual bool HasFocusedDescendants() const override
	{
		return false;
	}

	virtual bool HasFocusedDescendants(uint32 UserIndex) const override
	{
		return false;
	}

	virtual bool Exists() const override
	{
		return false;
	}

	virtual bool IsVisible() const override
	{
		return false;
	}

	virtual bool IsChecked() const override
	{
		return false;
	}

	virtual bool IsInteractable() const override
	{
		return false;
	}

	virtual bool IsScrollable() const override
	{
		return false;
	}

	virtual bool IsScrolledToBeginning() const override
	{
		return false;
	}

	virtual bool IsScrolledToEnd() const override
	{
		return false;
	}

	virtual bool IsHovered() const override
	{
		return false;
	}

	virtual FVector2D GetAbsolutePosition() const override
	{
		return FVector2D(0, 0);
	}

	virtual FVector2D GetSize() const override
	{
		return FVector2D(0, 0);
	}

	virtual FText GetText() const override
	{
		return FText::GetEmpty();
	}

	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const override
	{

	}

	virtual FString ToDebugString() const override
	{
		return TEXT("Empty Driver Element");
	}

private:

	FEmptyDriverElement()
	{ }

private:


	friend FEmptyDriverElementFactory;
};

TSharedRef<IDriverElement, ESPMode::ThreadSafe> FEmptyDriverElementFactory::Create()
{
	return MakeShareable(new FEmptyDriverElement());
}
