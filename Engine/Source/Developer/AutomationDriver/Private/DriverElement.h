// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IElementLocator;


class FAsyncAutomationDriver;
class IAsyncDriverElementCollection;
class IAsyncDriverElement;

class FAsyncDriverElementCollectionFactory
{
public:

	static TSharedRef<IAsyncDriverElementCollection, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver,
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator);
};

class FAsyncDriverElementFactory
{
public:

	static TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver,
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator);
};



class FAutomationDriver;
class IDriverElementCollection;
class IDriverElement;

class FDriverElementCollectionFactory
{
public:

	static TSharedRef<IDriverElementCollection, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe>& Driver,
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator);
};

class FDriverElementFactory
{
public:

	static TSharedRef<IDriverElement, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe>& Driver,
		const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator);
};