// Copyright Epic Games, Inc. All Rights Reserved.

#include "DriverSequence.h"
#include "IStepExecutor.h"
#include "IDriverSequence.h"
#include "IElementLocator.h"
#include "IApplicationElement.h"
#include "IApplicationElement.h"

#include "AutomatedApplication.h"
#include "AutomationDriverLogging.h"
#include "AutomationDriver.h"
#include "DriverConfiguration.h"
#include "StepExecutor.h"
#include "WaitUntil.h"
#include "LocateBy.h"

#include "GenericPlatform/ICursor.h"
#include "Misc/Timespan.h"
#include "InputCoreTypes.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "Framework/Application/SlateApplication.h"


class FStep
{
public:

	static FStepResult Done()
	{
		return FStepResult(FStepResult::EState::DONE, FTimespan::FromSeconds(0.01));
	}

	static FStepResult Done(double Seconds)
	{
		return FStepResult(FStepResult::EState::DONE, FTimespan::FromSeconds(Seconds));
	}

	static FStepResult Done(const FTimespan& Value)
	{
		return FStepResult(FStepResult::EState::DONE, Value);
	}

	static FStepResult Wait(double Seconds)
	{
		return FStepResult(FStepResult::EState::REPEAT, FTimespan::FromSeconds(Seconds));
	}

	static FStepResult Wait(const FTimespan& Value)
	{
		return FStepResult(FStepResult::EState::REPEAT, Value);
	}

	static FStepResult Failed()
	{
		return FStepResult(FStepResult::EState::FAILED, FTimespan::MinValue());
	}
};

class FActionSequenceExtensions
{
public:

	static bool InterpreteCharacter(TCHAR Character, int32* OutKeyCode, int32* OutCharCode)
	{
		bool bSuccess = false;
		FKey Key = FInputKeyManager::Get().GetKeyFromCodes(0, Character);

		if (!Key.IsValid())
		{
			if (Character == TEXT('\n'))
			{
				// Treat line feed characters as a simulated Enter key press
				Key = EKeys::Enter;
			}
			else if (Character == TEXT('\t'))
			{
				Key = EKeys::Tab;
			}
		}

		if (Key.IsValid())
		{
			const uint32* KeyCodePtr;
			const uint32* CharCodePtr;
			FInputKeyManager::Get().GetCodesFromKey(Key, KeyCodePtr, CharCodePtr);

			*OutKeyCode = (KeyCodePtr == nullptr) ? 0 : *KeyCodePtr;
			*OutCharCode = (CharCodePtr == nullptr) ? Character : *CharCodePtr;
			bSuccess = true;
		}
		else if (Character != TEXT('\r')) // skip processing any carriage returns
		{
			*OutKeyCode = 0;
			*OutCharCode = Character;
			bSuccess = true;
		}

		return bSuccess;
	}

	static FStepResult LocateElement(const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver, const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const FTimespan& TotalProcessTime, TSharedPtr<IApplicationElement>& OutElement)
	{
		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		if (Elements.Num() > 1)
		{
			FAutomationDriverLogging::TooManyElementsFound(Elements);
			return FStep::Failed();
		}

		if (Elements.Num() == 0)
		{
			if (TotalProcessTime >= AsyncDriver->GetConfiguration()->ImplicitWait)
			{
				FAutomationDriverLogging::CannotFindElement(ElementLocator);
				return FStep::Failed();
			}

			return FStep::Wait(1);
		}

		OutElement = Elements[0];
		return FStep::Done();
	}

	static FStepResult LocateVisibleElement(const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver, const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const FTimespan& TotalProcessTime, TSharedPtr<IApplicationElement>& OutElement)
	{
		TSharedPtr<IApplicationElement> Element;
		FStepResult Result = LocateElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

		if (Result.State != FStepResult::EState::DONE)
		{
			return Result;
		}

		if (!Element->IsVisible())
		{
			if (TotalProcessTime >= AsyncDriver->GetConfiguration()->ImplicitWait)
			{
				FAutomationDriverLogging::ElementNotVisible(ElementLocator);
				return FStep::Failed();
			}

			return FStep::Wait(1);
		}

		OutElement = Element;
		return FStep::Done();
	}

	static FStepResult LocateVisibleInteractableElement(const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver, const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const FTimespan& TotalProcessTime, TSharedPtr<IApplicationElement>& OutElement)
	{
		TSharedPtr<IApplicationElement> Element;
		FStepResult Result = LocateVisibleElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

		if (Result.State != FStepResult::EState::DONE)
		{
			return Result;
		}

		if (!Element->IsInteractable())
		{
			if (TotalProcessTime >= AsyncDriver->GetConfiguration()->ImplicitWait)
			{
				FAutomationDriverLogging::ElementNotInteractable(ElementLocator);
				return FStep::Failed();
			}

			return FStep::Wait(1);
		}

		OutElement = Element;
		return FStep::Done();
	}
};

class FAsyncActionSequence
	: public IAsyncActionSequence
	, public TSharedFromThis<FAsyncActionSequence, ESPMode::ThreadSafe>
{
public:

	enum class EElementAnchor : uint8
	{
		TOP_LEFT_CORNER,
		CENTER,
	};

	virtual ~FAsyncActionSequence()
	{ }

	bool IsExecuting() override
	{
		return StepsExecutor->IsExecuting();
	}

	virtual IAsyncActionSequence& Wait(FTimespan Timespan) override
	{
		InternalWait(Until::Lambda([Timespan](const FTimespan& TotalWaitTime) {
			check(IsInGameThread());

			if (TotalWaitTime > Timespan)
			{
				return FDriverWaitResponse::Passed();
			}

			return FDriverWaitResponse::Wait(Timespan);
		}));
		return *this;
	}

	virtual IAsyncActionSequence& Wait(const FDriverWaitDelegate& Delegate) override
	{
		InternalWait(Delegate);
		return *this;
	}

	virtual IAsyncActionSequence& MoveToElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, int32 XOffset, int32 YOffset) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, XOffset, YOffset);
		return *this;
	}

	virtual IAsyncActionSequence& MoveToElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		return *this;
	}

	virtual IAsyncActionSequence& MoveByOffset(int32 XOffset, int32 YOffset) override
	{
		InternalMoveByOffset(XOffset, YOffset);
		return *this;
	}

	virtual IAsyncActionSequence& MoveToPosition(const FVector2D& Position) override
	{
		InternalMoveToPosition(Position);
		return *this;
	}

	virtual IAsyncActionSequence& ScrollBy(float Delta) override
	{
		InternalScrollBy(Delta);
		return *this;
	}

	virtual IAsyncActionSequence& ScrollBy(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Delta) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalScrollBy(Delta);
		return *this;
	}

	virtual IAsyncActionSequence& ScrollToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalScrollToBeginning(ElementLocator, 999999);
		return *this;
	}

	virtual IAsyncActionSequence& ScrollToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Amount) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalScrollToBeginning(ElementLocator, FMath::Abs(Amount));
		return *this;
	}

	virtual IAsyncActionSequence& ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalScrollUntil(By::Cursor(), ElementLocator, 1);
		return *this;
	}

	virtual IAsyncActionSequence& ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ScrollableElementLocator, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalMoveToElement(ScrollableElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalScrollUntil(ScrollableElementLocator, ElementLocator, 1);
		return *this;
	}

	virtual IAsyncActionSequence& ScrollToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalScrollToEnd(ElementLocator, -999999);
		return *this;
	}

	virtual IAsyncActionSequence& ScrollToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Amount) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalScrollToEnd(ElementLocator, FMath::Abs(Amount) * -1);
		return *this;
	}

	virtual IAsyncActionSequence& ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalScrollUntil(By::Cursor(), ElementLocator, -1);
		return *this;
	}

	virtual IAsyncActionSequence& ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ScrollableElementLocator, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalMoveToElement(ScrollableElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalScrollUntil(ScrollableElementLocator, ElementLocator, -1);
		return *this;
	}

	virtual IAsyncActionSequence& Click(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalClick(ElementLocator, EMouseButtons::Left);
		return *this;
	}

	virtual IAsyncActionSequence& Click(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalClick(ElementLocator, MouseButton);
		return *this;
	}

	virtual IAsyncActionSequence& Click(EMouseButtons::Type MouseButton) override
	{
		InternalClick(By::Cursor(), MouseButton);
		return *this;
	}

	virtual IAsyncActionSequence& Click() override
	{
		Click(EMouseButtons::Left);
		return *this;
	}

	virtual IAsyncActionSequence& DoubleClick(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalDoubleClick(ElementLocator, EMouseButtons::Left);
		return *this;
	}

	virtual IAsyncActionSequence& DoubleClick(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalDoubleClick(ElementLocator, MouseButton);
		return *this;
	}

	virtual IAsyncActionSequence& DoubleClick(EMouseButtons::Type MouseButton) override
	{
		InternalDoubleClick(By::Cursor(), MouseButton);
		return *this;
	}

	virtual IAsyncActionSequence& DoubleClick() override
	{
		InternalDoubleClick(By::Cursor(), EMouseButtons::Left);
		return *this;
	}

	virtual IAsyncActionSequence& Type(const TCHAR* Text) override
	{
		Type(FString(Text));
		return *this;
	}

	virtual IAsyncActionSequence& Type(FString Text) override
	{
		FString Temp;

		for (TCHAR Char : Text.GetCharArray())
		{
			if (Char == TCHAR())
			{
				break;
			}

			Type(Char);
		}
		return *this;
	}

	virtual IAsyncActionSequence& Type(FKey Key) override
	{
		const uint32* KeyCodePtr;
		const uint32* CharCodePtr;
		FInputKeyManager::Get().GetCodesFromKey(Key, KeyCodePtr, CharCodePtr);

		int32 KeyCode = (KeyCodePtr == nullptr) ? 0 : *KeyCodePtr;
		int32 CharCode = (CharCodePtr == nullptr) ? 0 : *CharCodePtr;

		InternalSendKey(KeyCode, CharCode);
		return *this;
	}

	virtual IAsyncActionSequence& Type(TCHAR Character) override
	{
		int32 KeyCode;
		int32 CharCode;
		if (FActionSequenceExtensions::InterpreteCharacter(Character, &KeyCode, &CharCode))
		{
			InternalSendKey(KeyCode, CharCode);
		}

		return *this;
	}

	virtual IAsyncActionSequence& Type(const TArray<FKey>& Keys) override
	{
		for (const FKey& Key: Keys)
		{
			InternalSendKey(Key, 0);
		}
		return *this;
	}

	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const TCHAR* Text) override
	{
		InternalEnsureFocus(ElementLocator);
		Type(Text);
		return *this;
	}

	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FString Text) override
	{
		InternalEnsureFocus(ElementLocator);
		Type(MoveTemp(Text));
		return *this;
	}

	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key)  override
	{
		InternalEnsureFocus(ElementLocator);
		Type(Key);
		return *this;
	}

	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character)  override
	{
		InternalEnsureFocus(ElementLocator);
		Type(Character);
		return *this;
	}

	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const TArray<FKey>& Keys)  override
	{
		InternalEnsureFocus(ElementLocator);
		Type(Keys);
		return *this;
	}

	virtual IAsyncActionSequence& TypeChord(FKey Key1, FKey Key2) override
	{
		Press(Key1);
		Press(Key2);
		Release(Key2);
		Release(Key1);
		return *this;
	}

	virtual IAsyncActionSequence& TypeChord(FKey Key1, TCHAR Character) override
	{
		Press(Key1);
		Press(Character);
		Release(Character);
		Release(Key1);
		return *this;
	}

	virtual IAsyncActionSequence& TypeChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		Press(Key1);
		Press(Key2);
		Press(Key3);
		Release(Key3);
		Release(Key2);
		Release(Key1);
		return *this;
	}

	virtual IAsyncActionSequence& TypeChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		Press(Key1);
		Press(Key2);
		Press(Character);
		Release(Character);
		Release(Key2);
		Release(Key1);
		return *this;
	}

	virtual IAsyncActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2)  override
	{
		InternalEnsureFocus(ElementLocator);
		TypeChord(Key1, Key2);
		return *this;
	}

	virtual IAsyncActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character)  override
	{
		InternalEnsureFocus(ElementLocator);
		TypeChord(Key1, Character);
		return *this;
	}

	virtual IAsyncActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3)  override
	{
		InternalEnsureFocus(ElementLocator);
		TypeChord(Key1, Key2, Key3);
		return *this;
	}

	virtual IAsyncActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character)  override
	{
		InternalEnsureFocus(ElementLocator);
		TypeChord(Key1, Key2, Character);
		return *this;
	}

	virtual IAsyncActionSequence& Press(TCHAR Character) override
	{
		InternalPress(Character);
		return *this;
	}

	virtual IAsyncActionSequence& Press(FKey Key) override
	{
		InternalPress(Key, 0);
		return *this;
	}

	virtual IAsyncActionSequence& Press(EMouseButtons::Type MouseButton) override
	{
		InternalPress(By::Cursor(), MouseButton);
		return *this;
	}

	virtual IAsyncActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalPress(Character);
		return *this;
	}

	virtual IAsyncActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalPress(Key, 0);
		return *this;
	}

	virtual IAsyncActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalPress(ElementLocator, MouseButton);
		return *this;
	}

	virtual IAsyncActionSequence& PressChord(FKey Key1, FKey Key2) override
	{
		InternalPress(Key1, 0);
		InternalPress(Key2, 0);
		return *this;
	}

	virtual IAsyncActionSequence& PressChord(FKey Key1, TCHAR Character) override
	{
		InternalPress(Key1, 0);
		InternalPress(Character);
		return *this;
	}

	virtual IAsyncActionSequence& PressChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		InternalPress(Key1, 0);
		InternalPress(Key2, 0);
		InternalPress(Key3, 0);
		return *this;
	}

	virtual IAsyncActionSequence& PressChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		InternalPress(Key1, 0);
		InternalPress(Key2, 0);
		InternalPress(Character);
		return *this;
	}

	virtual IAsyncActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalPress(Key1, 0);
		InternalPress(Key2, 0);
		return *this;
	}

	virtual IAsyncActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalPress(Key1, 0);
		InternalPress(Character);
		return *this;
	}

	virtual IAsyncActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalPress(Key1, 0);
		InternalPress(Key2, 0);
		InternalPress(Key3, 0);
		return *this;
	}

	virtual IAsyncActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalPress(Key1, 0);
		InternalPress(Key2, 0);
		InternalPress(Character);
		return *this;
	}

	virtual IAsyncActionSequence& Release(TCHAR Character) override
	{
		InternalRelease(Character);
		return *this;
	}

	virtual IAsyncActionSequence& Release(FKey Key) override
	{
		InternalRelease(Key, 0);
		return *this;
	}

	virtual IAsyncActionSequence& Release(EMouseButtons::Type MouseButton) override
	{
		InternalRelease(MouseButton);
		return *this;
	}

	virtual IAsyncActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalRelease(Character);
		return *this;
	}

	virtual IAsyncActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalRelease(Key, 0);
		return *this;
	}

	virtual IAsyncActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) override
	{
		InternalMoveToElement(ElementLocator, EElementAnchor::CENTER, 0, 0);
		InternalRelease(MouseButton);
		return *this;
	}

	virtual IAsyncActionSequence& ReleaseChord(FKey Key1, FKey Key2) override
	{
		InternalRelease(Key2, 0);
		InternalRelease(Key1, 0);
		return *this;
	}

	virtual IAsyncActionSequence& ReleaseChord(FKey Key1, TCHAR Character) override
	{
		InternalRelease(Character);
		InternalRelease(Key1, 0);
		return *this;
	}

	virtual IAsyncActionSequence& ReleaseChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		InternalRelease(Key3, 0);
		InternalRelease(Key2, 0);
		InternalRelease(Key1, 0);
		return *this;
	}

	virtual IAsyncActionSequence& ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		InternalRelease(Character);
		InternalRelease(Key2, 0);
		InternalRelease(Key1, 0);
		return *this;
	}

	virtual IAsyncActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalRelease(Key2, 0);
		InternalRelease(Key1, 0);
		return *this;
	}

	virtual IAsyncActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalRelease(Character);
		InternalRelease(Key1, 0);
		return *this;
	}

	virtual IAsyncActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalRelease(Key3, 0);
		InternalRelease(Key2, 0);
		InternalRelease(Key1, 0);
		return *this;
	}

	virtual IAsyncActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) override
	{
		InternalEnsureFocus(ElementLocator);
		InternalRelease(Character);
		InternalRelease(Key2, 0);
		InternalRelease(Key1, 0);
		return *this;
	}

	virtual IAsyncActionSequence& Focus(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		InternalFocus(ElementLocator);
		return *this;
	}

	virtual IAsyncActionSequence& Focus(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, uint32 UserFocus) override
	{
		InternalFocus(ElementLocator, UserFocus);
		return *this;
	}

	TAsyncResult<bool> Perform()
	{
		if (!Application->IsHandlingMessages())
		{
			return false;
		}

		if (!AsyncDriver->PinActionSequence(SharedThis(this)))
		{
			return false;
		}

		TWeakPtr<FAsyncActionSequence, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));
		StepsExecutor->OnCompleted([LocalWeakThis]()
		{
			if (TSharedPtr<FAsyncActionSequence, ESPMode::ThreadSafe> Sequence = LocalWeakThis.Pin())
			{
				Sequence->AsyncDriver->UnpinActionSequence(Sequence);
			}
		});

		return StepsExecutor->Execute();
	}

private:

	FAsyncActionSequence(
		const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& InAsyncDriver,
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& InApplication,
		const TSharedRef<IStepExecutor, ESPMode::ThreadSafe>& InStepsExecutor)
		: AsyncDriver(InAsyncDriver)
		, Application(InApplication)
		, StepsExecutor(InStepsExecutor)
	{ }

	void InternalWait(const FDriverWaitDelegate& Delegate)
	{
		StepsExecutor->Add([this, Delegate](const FTimespan& TotalProcessTime) -> FStepResult {

			if (Delegate.IsBound())
			{
				FDriverWaitResponse Response = Delegate.Execute(TotalProcessTime);

				if (Response.State == FDriverWaitResponse::EState::WAIT)
				{
					return FStep::Wait(Response.NextWait);
				}
				else if (Response.State == FDriverWaitResponse::EState::FAILED)
				{
					return FStep::Failed();
				}
			}

			return FStep::Done();
		});
	}

	void InternalMoveToElement(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EElementAnchor Anchor, float XOffset, float YOffset)
	{
		StepsExecutor->Add([this, ElementLocator, Anchor, XOffset, YOffset](const FTimespan& EnsureElementTotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateElement(AsyncDriver, ElementLocator, EnsureElementTotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			if (!Element->IsVisible())
			{
				const TSharedPtr<IApplicationElement> ParentElement = Element->GetScrollableParent();

				if (ParentElement.IsValid())
				{
					const TSharedRef<IElementLocator, ESPMode::ThreadSafe> ParentElementLocator = ParentElement->CreateLocator();
					StepsExecutor->InsertNext([this, ElementLocator, ParentElementLocator, Anchor, XOffset, YOffset](const FTimespan& ScrollToElementTotalProcessTime) -> FStepResult {
						TSharedPtr<IApplicationElement> ScrollableParentElement;
						FStepResult ParentResult = FActionSequenceExtensions::LocateElement(AsyncDriver, ParentElementLocator, ScrollToElementTotalProcessTime, ScrollableParentElement);

						if (ParentResult.State != FStepResult::EState::DONE)
						{
							return ParentResult;
						}

						TSharedPtr<IApplicationElement> DesiredElement;
						FStepResult ChildResult = FActionSequenceExtensions::LocateElement(AsyncDriver, ElementLocator, ScrollToElementTotalProcessTime, DesiredElement);

						if (ChildResult.State != FStepResult::EState::DONE)
						{
							return ChildResult;
						}

						const FVector2D Position = ScrollableParentElement->GetAbsolutePosition();
						FVector2D CursorPosition(Position.X + XOffset, Position.Y + YOffset);

						if (Anchor == EElementAnchor::CENTER)
						{
							const FVector2D Size = ScrollableParentElement->GetSize();
							CursorPosition.X += Size.X / 2;
							CursorPosition.Y += Size.Y / 2;
						}
						else if (Anchor == EElementAnchor::TOP_LEFT_CORNER)
						{
							// do nothing
						}
						else
						{
							check(TEXT("Unknown MoveToElement Anchor specified"));
							return FStep::Failed();
						}

						Application->Cursor->SetPosition(CursorPosition.X, CursorPosition.Y);
						Application->GetRealMessageHandler()->OnMouseMove();

						const FVector2D ChildLocation = DesiredElement->GetAbsolutePosition();

						EOrientation ScrollOrientation;
						if (ScrollableParentElement->IsScrollable(ScrollOrientation))
						{
							if (ScrollOrientation == Orient_Horizontal)
							{
								if (ChildLocation.X < CursorPosition.X)
								{
									InternalScrollUntil(ParentElementLocator, ElementLocator, 1);
								}
								else if (ChildLocation.X > CursorPosition.X)
								{
									InternalScrollUntil(ParentElementLocator, ElementLocator, -1);
								}
							}
							else if (ScrollOrientation == Orient_Vertical)
							{
								if (ChildLocation.Y < CursorPosition.Y)
								{
									InternalScrollUntil(ParentElementLocator, ElementLocator, 1);
								}
								else if (ChildLocation.Y > CursorPosition.Y)
								{
									InternalScrollUntil(ParentElementLocator, ElementLocator, -1);
								}
							}
						}

						return FStep::Done();
					});
				}
			}

			return FStep::Done();
		});

		// Move the cursor to the desired element, even if not visible
		StepsExecutor->Add([this, ElementLocator, Anchor, XOffset, YOffset](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			const FVector2D Position = Element->GetAbsolutePosition();
			FVector2D CursorPosition(Position.X + XOffset, Position.Y + YOffset);

			if (Anchor == EElementAnchor::CENTER)
			{
				const FVector2D Size = Element->GetSize();
				CursorPosition.X += Size.X / 2;
				CursorPosition.Y += Size.Y / 2;
			}
			else if (Anchor == EElementAnchor::TOP_LEFT_CORNER)
			{
				// do nothing
			}
			else
			{
				check(TEXT("Unknown MoveToElement Anchor specified"));
				return FStep::Failed();
			}

			Application->Cursor->SetPosition(CursorPosition.X, CursorPosition.Y);
			Application->GetRealMessageHandler()->OnMouseMove();

			return FStep::Done();
		});
	}

	void InternalMoveByOffset(float XOffset, float YOffset)
	{
		StepsExecutor->Add([this, XOffset, YOffset](const FTimespan& TotalProcessTime) -> FStepResult {

			FVector2D CurrentPosition = Application->Cursor->GetPosition();

			Application->Cursor->SetPosition(CurrentPosition.X + XOffset, CurrentPosition.Y + YOffset);
			Application->GetRealMessageHandler()->OnMouseMove();

			return FStep::Done();
		});
	}

	void InternalMoveToPosition(const FVector2D& Position)
	{
		StepsExecutor->Add([this, Position](const FTimespan& TotalProcessTime) -> FStepResult {

			Application->Cursor->SetPosition(Position.X, Position.Y);
			Application->GetRealMessageHandler()->OnMouseMove();

			return FStep::Done();
			});
	}

	void InternalScrollBy(float Delta)
	{
		StepsExecutor->Add([this, Delta](const FTimespan& TotalProcessTime) -> FStepResult {
			Application->GetRealMessageHandler()->OnMouseWheel(Delta);
			return FStep::Done();
		});
	}

	void InternalScrollToBeginning(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Delta)
	{
		StepsExecutor->Add([this, ElementLocator, Delta](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateVisibleElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			if (Element->IsScrolledToBeginning())
			{
				return FStep::Done();
			}

			Application->GetRealMessageHandler()->OnMouseWheel(Delta);
			return FStep::Wait(FTimespan::Zero());
		});
	}

	void InternalScrollToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Delta)
	{
		StepsExecutor->Add([this, ElementLocator, Delta](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateVisibleElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			if (Element->IsScrolledToEnd())
			{
				return FStep::Done();
			}

			Application->GetRealMessageHandler()->OnMouseWheel(Delta);
			return FStep::Wait(FTimespan::Zero());
		});
	}

	void InternalScrollUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ScrollableElementLocator, const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Delta)
	{
		const TFunction<FStepResult(const FTimespan&)> Step = [this, ScrollableElementLocator, ElementLocator, Delta](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateVisibleElement(AsyncDriver, ElementLocator, FTimespan::Zero(), Element);

			if (Result.State == FStepResult::EState::DONE)
			{
				return Result;
			}

			TSharedPtr<IApplicationElement> ScrollableElement;
			Result = FActionSequenceExtensions::LocateVisibleElement(AsyncDriver, ScrollableElementLocator, TotalProcessTime, ScrollableElement);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			if (Delta < 0.0f && ScrollableElement->IsScrolledToEnd())
			{
				return FStep::Failed();
			}
			else if (!(Delta < 0.0f) && ScrollableElement->IsScrolledToBeginning())
			{
				return FStep::Failed();
			}

			Application->GetRealMessageHandler()->OnMouseWheel(Delta);
			return FStep::Wait(FTimespan::Zero());
		};

		// We add a small wait after moving the element into view in order to give the UI enough time to fully process the last wheel event
		// Otherwise, we occasionally have element scroll out from under the cursor on accident when performing follow up click
		const TFunction<FStepResult(const FTimespan&)> Wait = [this](const FTimespan& TotalProcessTime) -> FStepResult {
			if (TotalProcessTime < FTimespan::FromSeconds(0.5))
			{
				return FStep::Wait(FTimespan::FromSeconds(0.5));
			}

			return FStep::Done();
		};

		if (StepsExecutor->IsExecuting())
		{
			StepsExecutor->InsertNext(Wait);
			StepsExecutor->InsertNext(Step);
		}
		else
		{
			StepsExecutor->Add(Step);
			StepsExecutor->Add(Wait);
		}
	}

	void InternalClick(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton)
	{
		InternalPress(ElementLocator, MouseButton);
		InternalRelease(MouseButton);
	}

	void InternalDoubleClick(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton)
	{
		InternalActivateWindow(ElementLocator);

		// Send mouse down event
		StepsExecutor->Add([this, ElementLocator, MouseButton](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateVisibleInteractableElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			if (!Element->IsHovered())
			{
				if (TotalProcessTime >= AsyncDriver->GetConfiguration()->ImplicitWait)
				{
					FAutomationDriverLogging::CannotClickUnhoveredElement(ElementLocator);
					return FStep::Failed();
				}

				return FStep::Wait(1);
			}

			const TSharedPtr<FGenericWindow> Window = Element->GetWindow();
			if (!Window.IsValid())
			{
				FAutomationDriverLogging::ElementHasNoWindow(ElementLocator);
				return FStep::Failed();
			}

			Application->GetRealMessageHandler()->OnMouseDown(Window.ToSharedRef(), MouseButton);
			AsyncDriver->TrackPress(MouseButton);

			TWeakPtr<FGenericWindow> WeakWindow(Window);

			// Send double click event
			StepsExecutor->InsertNext([this, WeakWindow, MouseButton](const FTimespan& DoubleClickEventTotalProcessTime) -> FStepResult {
				const TSharedPtr<FGenericWindow> TargetWindow = WeakWindow.Pin();

				if (TargetWindow.IsValid())
				{
					Application->GetRealMessageHandler()->OnMouseDoubleClick(TargetWindow.ToSharedRef(), MouseButton);
					
					// Send mouse up event
					StepsExecutor->InsertNext([this, MouseButton](const FTimespan& SecondMouseUpEventTotalProcessTime) -> FStepResult {
						Application->GetRealMessageHandler()->OnMouseUp(MouseButton);
						AsyncDriver->TrackRelease(MouseButton);

						return FStep::Done(0);
					});
				}

				return FStep::Done(0);
			});

			// Send mouse up event
			StepsExecutor->InsertNext([this, MouseButton](const FTimespan& FirstMouseUpEventTotalProcessTime) -> FStepResult {
				Application->GetRealMessageHandler()->OnMouseUp(MouseButton);
				AsyncDriver->TrackRelease(MouseButton);

				return FStep::Done(0);
			});

			return FStep::Done(0);
		});
	}

	void InternalSendKey(FKey Key, TCHAR Char)
	{
		const uint32* KeyCodePtr;
		const uint32* CharCodePtr;
		FInputKeyManager::Get().GetCodesFromKey(Key, KeyCodePtr, CharCodePtr);

		int32 KeyCode = (KeyCodePtr == nullptr) ? 0 : *KeyCodePtr;
		int32 CharCode = (CharCodePtr == nullptr) ? Char : *CharCodePtr;

		InternalSendKey(KeyCode, CharCode);
	}

	void InternalSendKey(int32 KeyCode, int32 CharCode)
	{
		InternalPress(KeyCode, CharCode);
		InternalRelease(KeyCode, CharCode);
	}

	void InternalActivateWindow(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		// Activate the window
		StepsExecutor->Add([this, ElementLocator](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			const TSharedPtr<FGenericWindow> Window = Element->GetWindow();
			if (!Window.IsValid())
			{
				FAutomationDriverLogging::ElementHasNoWindow(ElementLocator);
				return FStep::Failed();
			}

			TSharedPtr<SWindow> ActiveWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
			if (!ActiveWindow.IsValid() || ActiveWindow->GetNativeWindow() != Window)
			{
				Window->SetWindowFocus();
			}

			return FStep::Done();
		});
	}

	void InternalPress(TCHAR Character)
	{
		int32 KeyCode;
		int32 CharCode;
		if (FActionSequenceExtensions::InterpreteCharacter(Character, &KeyCode, &CharCode))
		{
			InternalPress(KeyCode, CharCode);
		}
	}

	void InternalPress(FKey Key, TCHAR Character)
	{
		const uint32* KeyCodePtr;
		const uint32* CharCodePtr;
		FInputKeyManager::Get().GetCodesFromKey(Key, KeyCodePtr, CharCodePtr);

		int32 KeyCode = (KeyCodePtr == nullptr) ? 0 : *KeyCodePtr;
		int32 CharCode = (CharCodePtr == nullptr) ? Character : *CharCodePtr;

		InternalPress(KeyCode, CharCode);
	}

	void InternalPress(int32 KeyCode, int32 CharCode)
	{
		// Send key down event
		StepsExecutor->Add([this, KeyCode, CharCode](const FTimespan& TotalProcessTime) -> FStepResult {
			if (!AsyncDriver->IsPressed(KeyCode, CharCode))
			{
				Application->GetRealMessageHandler()->OnKeyDown(KeyCode, CharCode, false);
				AsyncDriver->TrackPress(KeyCode, CharCode);
			}

			return FStep::Done();
		});

		if (CharCode != 0)
		{
			InternalCharKey(CharCode, false);
		}
	}

	void InternalPress(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton)
	{
		InternalActivateWindow(ElementLocator);

		// Send mouse down event
		StepsExecutor->Add([this, ElementLocator, MouseButton](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateVisibleInteractableElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			if (!Element->IsHovered())
			{
				if (TotalProcessTime >= AsyncDriver->GetConfiguration()->ImplicitWait)
				{
					FAutomationDriverLogging::CannotClickUnhoveredElement(ElementLocator);
					return FStep::Failed();
				}

				return FStep::Wait(1);
			}

			const TSharedPtr<FGenericWindow> Window = Element->GetWindow();
			if (!Window.IsValid())
			{
				FAutomationDriverLogging::ElementHasNoWindow(ElementLocator);
				return FStep::Failed();
			}

			if (!AsyncDriver->IsPressed(MouseButton))
			{
				Application->GetRealMessageHandler()->OnMouseDown(Window.ToSharedRef(), MouseButton);
				AsyncDriver->TrackPress(MouseButton);
			}

			return FStep::Done();
		});
	}

	void InternalCharKey(TCHAR Character, bool bIsRepeat)
	{
		// Send key char event
		StepsExecutor->Add([this, Character, bIsRepeat](const FTimespan& TotalProcessTime) -> FStepResult {
			TCHAR FinalCharacter = AsyncDriver->ProcessCharacterForControlCodes(Character);
			Application->GetRealMessageHandler()->OnKeyChar(FinalCharacter, bIsRepeat);
			return FStep::Done();
		});
	}

	void InternalRelease(TCHAR Character)
	{
		int32 KeyCode;
		int32 CharCode;
		if (FActionSequenceExtensions::InterpreteCharacter(Character, &KeyCode, &CharCode))
		{
			InternalRelease(KeyCode, CharCode);
		}
	}

	void InternalRelease(FKey Key, TCHAR Char)
	{
		const uint32* KeyCodePtr;
		const uint32* CharCodePtr;
		FInputKeyManager::Get().GetCodesFromKey(Key, KeyCodePtr, CharCodePtr);

		int32 KeyCode = (KeyCodePtr == nullptr) ? 0 : *KeyCodePtr;
		int32 CharCode = (CharCodePtr == nullptr) ? Char : *CharCodePtr;

		InternalRelease(KeyCode, CharCode);
	}

	void InternalRelease(int32 KeyCode, int32 CharCode)
	{
		// Send key up event
		StepsExecutor->Add([this, KeyCode, CharCode](const FTimespan& TotalProcessTime) -> FStepResult {
			if (AsyncDriver->IsPressed(KeyCode, CharCode))
			{
				Application->GetRealMessageHandler()->OnKeyUp(KeyCode, CharCode, false);
				AsyncDriver->TrackRelease(KeyCode, CharCode);
			}
			return FStep::Done();
		});
	}

	void InternalRelease(EMouseButtons::Type MouseButton)
	{
		// Send mouse up event
		StepsExecutor->Add([this, MouseButton](const FTimespan& TotalProcessTime) -> FStepResult {
			if (AsyncDriver->IsPressed(MouseButton))
			{
				Application->GetRealMessageHandler()->OnMouseUp(MouseButton);
				AsyncDriver->TrackRelease(MouseButton);
			}
			return FStep::Done();
		});
	}

	void InternalEnsureFocus(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		StepsExecutor->Add([this, ElementLocator](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateVisibleElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			if (!Element->HasFocusedDescendants())
			{
				Element->Focus();
			}

			return FStep::Done();
		});
	}

	void InternalFocus(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator)
	{
		StepsExecutor->Add([this, ElementLocator](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateVisibleElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			if (!Element->IsFocused())
			{
				Element->Focus();
			}

			return FStep::Done();
		});
	}

	void InternalFocus(const TSharedPtr<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, uint32 UserIndex)
	{
		StepsExecutor->Add([this, ElementLocator, UserIndex](const FTimespan& TotalProcessTime) -> FStepResult {
			TSharedPtr<IApplicationElement> Element;
			FStepResult Result = FActionSequenceExtensions::LocateVisibleElement(AsyncDriver, ElementLocator, TotalProcessTime, Element);

			if (Result.State != FStepResult::EState::DONE)
			{
				return Result;
			}

			if (!Element->IsFocused())
			{
				Element->Focus(UserIndex);
			}

			return FStep::Done();
		});
	}

private:

	const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe> AsyncDriver;
	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe> Application;
	const TSharedRef<IStepExecutor, ESPMode::ThreadSafe> StepsExecutor;

	friend FAsyncActionSequenceFactory;
};

TSharedRef<FAsyncActionSequence, ESPMode::ThreadSafe> FAsyncActionSequenceFactory::Create(
	const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver,
	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& Application)
{
	return MakeShareable(new FAsyncActionSequence(
		AsyncDriver,
		Application,
		FStepExecutorFactory::Create(AsyncDriver->GetConfiguration(), Application)));
}

class FAsyncDriverSequence
	: public IAsyncDriverSequence
{
public:

	virtual ~FAsyncDriverSequence()
	{ }

	virtual IAsyncActionSequence& Actions()
	{
		return *ActionSequence;
	}

	virtual TAsyncResult<bool> Perform()
	{
		return ActionSequence->Perform();
	}

	FAsyncDriverSequence(
		const TSharedRef<FAsyncActionSequence, ESPMode::ThreadSafe>& InActionSequence)
		: ActionSequence(InActionSequence)
	{ }

private:

	const TSharedRef<FAsyncActionSequence, ESPMode::ThreadSafe> ActionSequence;

	friend FAsyncDriverSequenceFactory;
};

TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> FAsyncDriverSequenceFactory::Create(
	const TSharedRef<FAsyncActionSequence, ESPMode::ThreadSafe>& ActionSequence)
{
	return MakeShareable(new FAsyncDriverSequence(
		ActionSequence));
}



class FActionSequence
	: public IActionSequence
	, public TSharedFromThis<FActionSequence, ESPMode::ThreadSafe>
{
public:

	virtual ~FActionSequence()
	{ }

	virtual IActionSequence& Wait(FTimespan Timespan) override
	{
		ActionSequence->Wait(Timespan);
		return *this;
	}

	virtual IActionSequence& Wait(const FDriverWaitDelegate& Delegate) override
	{
		ActionSequence->Wait(Delegate);
		return *this;
	}

	virtual IActionSequence& MoveToElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, int32 XOffset, int32 YOffset) override
	{
		ActionSequence->MoveToElement(ElementLocator, XOffset, YOffset);
		return *this;
	}

	virtual IActionSequence& MoveToElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->MoveToElement(ElementLocator);
		return *this;
	}

	virtual IActionSequence& MoveByOffset(int32 XOffset, int32 YOffset) override
	{
		ActionSequence->MoveByOffset(XOffset, YOffset);
		return *this;
	}

	virtual IActionSequence& MoveToPosition(const FVector2D& Position) override
	{
		ActionSequence->MoveToPosition(Position);
		return *this;
	}

	virtual IActionSequence& ScrollBy(float Delta) override
	{
		ActionSequence->ScrollBy(Delta);
		return *this;
	}

	virtual IActionSequence& ScrollBy(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Delta) override
	{
		ActionSequence->ScrollBy(ElementLocator, Delta);
		return *this;
	}

	virtual IActionSequence& ScrollToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->ScrollToBeginning(ElementLocator);
		return *this;
	}

	virtual IActionSequence& ScrollToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Amount) override
	{
		ActionSequence->ScrollToBeginning(ElementLocator, Amount);
		return *this;
	}

	virtual IActionSequence& ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->ScrollToBeginningUntil(ElementLocator);
		return *this;
	}

	virtual IActionSequence& ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ScrollableElementLocator, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->ScrollToBeginningUntil(ScrollableElementLocator, ElementLocator);
		return *this;
	}

	virtual IActionSequence& ScrollToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->ScrollToEnd(ElementLocator);
		return *this;
	}

	virtual IActionSequence& ScrollToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Amount) override
	{
		ActionSequence->ScrollToEnd(ElementLocator, Amount);
		return *this;
	}

	virtual IActionSequence& ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->ScrollToEndUntil(ElementLocator);
		return *this;
	}

	virtual IActionSequence& ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ScrollableElementLocator, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->ScrollToEndUntil(ScrollableElementLocator, ElementLocator);
		return *this;
	}

	virtual IActionSequence& Click(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->Click(ElementLocator);
		return *this;
	}

	virtual IActionSequence& Click(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) override
	{
		ActionSequence->Click(ElementLocator, MouseButton);
		return *this;
	}

	virtual IActionSequence& Click(EMouseButtons::Type MouseButton) override
	{
		ActionSequence->Click(MouseButton);
		return *this;
	}

	virtual IActionSequence& Click() override
	{
		ActionSequence->Click();
		return *this;
	}

	virtual IActionSequence& DoubleClick(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->DoubleClick(ElementLocator);
		return *this;
	}
	
	virtual IActionSequence& DoubleClick(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) override
	{
		ActionSequence->DoubleClick(ElementLocator, MouseButton);
		return *this;
	}
	
	virtual IActionSequence& DoubleClick(EMouseButtons::Type MouseButton) override
	{
		ActionSequence->DoubleClick(MouseButton);
		return *this;
	}

	virtual IActionSequence& DoubleClick() override
	{
		ActionSequence->DoubleClick();
		return *this;
	}

	virtual IActionSequence& Type(const TCHAR* Text) override
	{
		ActionSequence->Type(Text);
		return *this;
	}

	virtual IActionSequence& Type(FString Text) override
	{
		ActionSequence->Type(MoveTemp(Text));
		return *this;
	}

	virtual IActionSequence& Type(FKey Key) override
	{
		ActionSequence->Type(Key);
		return *this;
	}

	virtual IActionSequence& Type(TCHAR Character) override
	{
		ActionSequence->Type(Character);
		return *this;
	}

	virtual IActionSequence& Type(const TArray<FKey>& Keys) override
	{
		ActionSequence->Type(Keys);
		return *this;
	}

	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const TCHAR* Text) override
	{
		ActionSequence->Type(ElementLocator, Text);
		return *this;
	}

	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FString Text) override
	{
		ActionSequence->Type(ElementLocator, MoveTemp(Text));
		return *this;
	}

	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key)  override
	{
		ActionSequence->Type(ElementLocator, Key);
		return *this;
	}

	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character)  override
	{
		ActionSequence->Type(ElementLocator, Character);
		return *this;
	}

	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const TArray<FKey>& Keys)  override
	{
		ActionSequence->Type(ElementLocator, Keys);
		return *this;
	}

	virtual IActionSequence& TypeChord(FKey Key1, FKey Key2) override
	{
		ActionSequence->TypeChord(Key1, Key2);
		return *this;
	}

	virtual IActionSequence& TypeChord(FKey Key1, TCHAR Character) override
	{
		ActionSequence->TypeChord(Key1, Character);
		return *this;
	}

	virtual IActionSequence& TypeChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		ActionSequence->TypeChord(Key1, Key2, Key3);
		return *this;
	}

	virtual IActionSequence& TypeChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		ActionSequence->TypeChord(Key1, Key2, Character);
		return *this;
	}

	virtual IActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2)  override
	{
		ActionSequence->TypeChord(ElementLocator, Key1, Key2);
		return *this;
	}

	virtual IActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character)  override
	{
		ActionSequence->TypeChord(ElementLocator, Key1, Character);
		return *this;
	}

	virtual IActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3)  override
	{
		ActionSequence->TypeChord(ElementLocator, Key1, Key2, Key3);
		return *this;
	}

	virtual IActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character)  override
	{
		ActionSequence->TypeChord(ElementLocator, Key1, Key2, Character);
		return *this;
	}

	virtual IActionSequence& Press(TCHAR Character) override
	{
		ActionSequence->Press(Character);
		return *this;
	}

	virtual IActionSequence& Press(FKey Key) override
	{
		ActionSequence->Press(Key);
		return *this;
	}

	virtual IActionSequence& Press(EMouseButtons::Type MouseButton) override
	{
		ActionSequence->Press(MouseButton);
		return *this;
	}

	virtual IActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) override
	{
		ActionSequence->Press(ElementLocator, Character);
		return *this;
	}

	virtual IActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key) override
	{
		ActionSequence->Press(ElementLocator, Key);
		return *this;
	}

	virtual IActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) override
	{
		ActionSequence->Press(ElementLocator, MouseButton);
		return *this;
	}

	virtual IActionSequence& PressChord(FKey Key1, FKey Key2) override
	{
		ActionSequence->PressChord(Key1, Key2);
		return *this;
	}

	virtual IActionSequence& PressChord(FKey Key1, TCHAR Character) override
	{
		ActionSequence->PressChord(Key1, Character);
		return *this;
	}

	virtual IActionSequence& PressChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		ActionSequence->PressChord(Key1, Key2, Key3);
		return *this;
	}

	virtual IActionSequence& PressChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		ActionSequence->PressChord(Key1, Key2, Character);
		return *this;
	}

	virtual IActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) override
	{
		ActionSequence->PressChord(ElementLocator, Key1, Key2);
		return *this;
	}

	virtual IActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) override
	{
		ActionSequence->PressChord(ElementLocator, Key1, Character);
		return *this;
	}

	virtual IActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) override
	{
		ActionSequence->PressChord(ElementLocator, Key1, Key2, Key3);
		return *this;
	}

	virtual IActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) override
	{
		ActionSequence->PressChord(ElementLocator, Key1, Key2, Character);
		return *this;
	}

	virtual IActionSequence& Release(TCHAR Character) override
	{
		ActionSequence->Release(Character);
		return *this;
	}

	virtual IActionSequence& Release(FKey Key) override
	{
		ActionSequence->Release(Key);
		return *this;
	}

	virtual IActionSequence& Release(EMouseButtons::Type MouseButton) override
	{
		ActionSequence->Release(MouseButton);
		return *this;
	}

	virtual IActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) override
	{
		ActionSequence->Release(ElementLocator, Character);
		return *this;
	}

	virtual IActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key) override
	{
		ActionSequence->Release(ElementLocator, Key);
		return *this;
	}

	virtual IActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) override
	{
		ActionSequence->Release(ElementLocator, MouseButton);
		return *this;
	}

	virtual IActionSequence& ReleaseChord(FKey Key1, FKey Key2) override
	{
		ActionSequence->ReleaseChord(Key1, Key2);
		return *this;
	}

	virtual IActionSequence& ReleaseChord(FKey Key1, TCHAR Character) override
	{
		ActionSequence->ReleaseChord(Key1, Character);
		return *this;
	}

	virtual IActionSequence& ReleaseChord(FKey Key1, FKey Key2, FKey Key3) override
	{
		ActionSequence->ReleaseChord(Key1, Key2, Key3);
		return *this;
	}

	virtual IActionSequence& ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) override
	{
		ActionSequence->ReleaseChord(Key1, Key2, Character);
		return *this;
	}

	virtual IActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) override
	{
		ActionSequence->ReleaseChord(ElementLocator, Key1, Key2);
		return *this;
	}

	virtual IActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) override
	{
		ActionSequence->ReleaseChord(ElementLocator, Key1, Character);
		return *this;
	}

	virtual IActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) override
	{
		ActionSequence->ReleaseChord(ElementLocator, Key1, Key2, Key3);
		return *this;
	}

	virtual IActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) override
	{
		ActionSequence->ReleaseChord(ElementLocator, Key1, Key2, Character);
		return *this;
	}

	virtual IActionSequence& Focus(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) override
	{
		ActionSequence->Focus(ElementLocator);
		return *this;
	}

	virtual IActionSequence& Focus(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, uint32 UserFocus) override
	{
		ActionSequence->Focus(ElementLocator, UserFocus);
		return *this;
	}

	bool Perform()
	{
		return ActionSequence->Perform().GetFuture().Get();
	}

private:

	FActionSequence(
		const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe>& InDriver,
		const TSharedRef<FAsyncActionSequence, ESPMode::ThreadSafe>& InActionSequence)
		: Driver(InDriver)
		, ActionSequence(InActionSequence)
	{ }


private:

	const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe> Driver;
	const TSharedRef<FAsyncActionSequence, ESPMode::ThreadSafe> ActionSequence;

	friend FActionSequenceFactory;
};

TSharedRef<FActionSequence, ESPMode::ThreadSafe> FActionSequenceFactory::Create(
	const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe>& Driver,
	const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver,
	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& Application)
{
	return MakeShareable(new FActionSequence(
		Driver,
		FAsyncActionSequenceFactory::Create(AsyncDriver, Application)));
}

class FDriverSequence
	: public IDriverSequence
{
public:

	virtual ~FDriverSequence()
	{ }

	virtual IActionSequence& Actions()
	{
		return *ActionSequence;
	}

	virtual bool Perform()
	{
		return ActionSequence->Perform();
	}

	FDriverSequence(
		const TSharedRef<FActionSequence, ESPMode::ThreadSafe>& InActionSequence)
		: ActionSequence(InActionSequence)
	{ }

private:

	const TSharedRef<FActionSequence, ESPMode::ThreadSafe> ActionSequence;

	friend FDriverSequenceFactory;
};

TSharedRef<IDriverSequence, ESPMode::ThreadSafe> FDriverSequenceFactory::Create(
	const TSharedRef<FActionSequence, ESPMode::ThreadSafe>& ActionSequence)
{
	return MakeShareable(new FDriverSequence(
		ActionSequence));
}
