// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FAutomatedApplication;


class FAsyncActionSequence;
class IAsyncDriverSequence;
class FAsyncAutomationDriver;
class FAsyncActionSequence;

class FAsyncActionSequenceFactory
{
public:

	static TSharedRef<FAsyncActionSequence, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver,
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& Application);
};

class FAsyncDriverSequenceFactory
{
public:

	static TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAsyncActionSequence, ESPMode::ThreadSafe>& ActionSequence);
};


class IDriverSequence;
class FAutomationDriver;
class FAsyncAutomationDriver;
class FActionSequence;

class FActionSequenceFactory
{
public:

	static TSharedRef<FActionSequence, ESPMode::ThreadSafe> Create(
		const TSharedRef<FAutomationDriver, ESPMode::ThreadSafe>& Driver,
		const TSharedRef<FAsyncAutomationDriver, ESPMode::ThreadSafe>& AsyncDriver,
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& Application);
};

class FDriverSequenceFactory
{
public:

	static TSharedRef<IDriverSequence, ESPMode::ThreadSafe> Create(
		const TSharedRef<FActionSequence, ESPMode::ThreadSafe>& ActionSequence);
};