// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/AsyncResult.h"
#include "Delegates/Delegate.h"
#include "Misc/Timespan.h"

struct FStepResult
{
public:

	enum class EState : uint8
	{
		DONE,
		FAILED,
		REPEAT,
	};

	FStepResult(EState InState, FTimespan InNextWait)
		: NextWait(MoveTemp(InNextWait))
		, State(InState)
	{ }

	// How long the executor should wait before either executing this same step, the next step or before declaring all steps complete
	FTimespan NextWait;

	// Whether the step that just completed is completely finished or should be rescheduled again for execution
	EState State;
};

class IStepExecutor
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(FStepResult, FExecuteStepDelegate, const FTimespan& /*TotalProcessTime*/);
	virtual void Add(const TSharedRef<FExecuteStepDelegate>& Step) = 0;

	virtual void Add(const TFunction<FStepResult(const FTimespan&)>& StepFunction) = 0;

	virtual void InsertNext(const TSharedRef<FExecuteStepDelegate>& Step) = 0;

	virtual void InsertNext(const TFunction<FStepResult(const FTimespan&)>& StepFunction) = 0;

	virtual TAsyncResult<bool> Execute() = 0;

	virtual bool IsExecuting() const = 0;

	virtual void OnCompleted(TFunction<void()> Callback) = 0;
};
