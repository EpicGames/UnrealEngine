// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocateBy.h"
#include "Locators/SlateWidgetLocatorByDelegate.h"
#include "Locators/SlateWidgetLocatorByPath.h"
#include "Locators/WidgetLocatorByFilter.h"
#include "AutomationDriverTypeDefs.h"
#include "Framework/Application/SlateApplication.h"
#include "IApplicationElement.h"

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Delegate(const FLocateSlateWidgetElementDelegate& Value, FStringView DebugName)
{
	return FSlateWidgetLocatorByDelegateFactory::Create(Value, DebugName);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Delegate(const FLocateSlateWidgetPathElementDelegate& Value, FStringView DebugName)
{
	return FSlateWidgetLocatorByDelegateFactory::Create(Value, DebugName);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::WidgetLambda(const TFunction<void(TArray<TSharedRef<SWidget>>&)>& Value, FStringView DebugName)
{
	return FSlateWidgetLocatorByDelegateFactory::Create(FLocateSlateWidgetElementDelegate::CreateLambda(Value), DebugName);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::WidgetPathLambda(const TFunction<void(TArray<FWidgetPath>&)>& Value, FStringView DebugName)
{
	return FSlateWidgetLocatorByDelegateFactory::Create(FLocateSlateWidgetPathElementDelegate::CreateLambda(Value), DebugName);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Id(const FString& Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(TEXT("#") + Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Id(const FDriverElementRef& Root, const FString& Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Root, TEXT("#") + Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Id(const FName& Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(TEXT("#") + Value.ToString());
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Id(const FDriverElementRef& Root, const FName& Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Root, TEXT("#") + Value.ToString());
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Id(const TCHAR* Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(FString(TEXT("#")) + Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Id(const FDriverElementRef& Root, const TCHAR* Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Root, FString(TEXT("#")) + Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Id(const char* Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(FString(TEXT("#")) + Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Id(const FDriverElementRef& Root, const char* Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Root, FString(TEXT("#")) + Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Path(const FString& Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Path(const FDriverElementRef& Root, const FString& Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Root, Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Path(const FName& Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Value.ToString());
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Path(const FDriverElementRef& Root, const FName& Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Root, Value.ToString());
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Path(const TCHAR* Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Path(const FDriverElementRef& Root, const TCHAR* Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Root, Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Path(const char* Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Path(const FDriverElementRef& Root, const char* Value)
{
	return FSlateWidgetLocatorByPathFactory::Create(Root, Value);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::Cursor()
{
	return By::WidgetPathLambda([](TArray<FWidgetPath>& OutWidgetPaths) -> void {
		TArray<TSharedRef<SWindow>> Windows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(Windows);
		FWidgetPath WidgetPath = FSlateApplication::Get().LocateWindowUnderMouse(FSlateApplication::Get().GetCursorPos(), Windows);

		if (WidgetPath.IsValid())
		{
			OutWidgetPaths.Add(WidgetPath);
		}
	});
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::KeyboardFocus()
{
	return WidgetLambda([](TArray<TSharedRef<SWidget>>& OutWidgets) -> void {
		if (const TSharedPtr<SWidget> Widget = FSlateApplication::Get().GetKeyboardFocusedWidget())
		{
			OutWidgets.Add(Widget.ToSharedRef());
		}
	});
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::UserFocus(uint32 UserIndex)
{
	return WidgetLambda([UserIndex](TArray<TSharedRef<SWidget>>& OutWidgets) -> void {
		if (const TSharedPtr<SWidget> Widget = FSlateApplication::Get().GetUserFocusedWidget(UserIndex))
		{
			OutWidgets.Add(Widget.ToSharedRef());
		}
	});
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::TextFilter::Contains(
	const FElementLocatorRef& RootLocator,
	const FString& Value,
	ESearchCase::Type SearchCase,
	ESearchDir::Type SearchDir)
{
	return FWidgetLocatorByFilterFactory::Create(
		TEXT("[By::TextFilter::Contains] ") + Value,
		RootLocator,
		[Value, SearchCase, SearchDir](const TSharedRef<IApplicationElement>& Element)
		{
			return Element->GetText().ToString().Contains(Value, SearchCase, SearchDir);
		}
	);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> By::TextFilter::Equals(
	const FElementLocatorRef& RootLocator,
	const FString& Value,
	ESearchCase::Type SearchCase)
{
	return FWidgetLocatorByFilterFactory::Create(
		TEXT("[By::TextFilter::Equals] ") + Value,
		RootLocator,
		[Value, SearchCase](const TSharedRef<IApplicationElement>& Element)
		{
			return Element->GetText().ToString().Equals(Value, SearchCase);
		}
	);
}
