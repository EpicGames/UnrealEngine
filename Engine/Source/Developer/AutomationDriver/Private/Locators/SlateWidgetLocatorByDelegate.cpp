// Copyright Epic Games, Inc. All Rights Reserved.

#include "Locators/SlateWidgetLocatorByDelegate.h"
#include "SlateWidgetElement.h"
#include "IElementLocator.h"
#include "Framework/Application/SlateApplication.h"

class FSlateWidgetLocatorByWidgetDelegate
	: public IElementLocator
{
public:

	virtual ~FSlateWidgetLocatorByWidgetDelegate()
	{ }

	virtual FString ToDebugString() const
	{
		static const FString TypeString = TEXT("[By::Delegate] ");
		if (!DebugName.IsEmpty())
		{
			return TypeString + DebugName;
		}

		FString DelegateName;
#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME
		DelegateName = Delegate.TryGetBoundFunctionName().ToString();
#endif
		return TypeString + DelegateName;
	}

	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const override
	{
		check(IsInGameThread());

		TArray<TSharedRef<SWidget>> Widgets;
		Delegate.Execute(Widgets);

		for (const TSharedRef<SWidget>& Widget : Widgets)
		{
			FWidgetPath WidgetPath;
			if (FSlateApplication::Get().FindPathToWidget(Widget, WidgetPath))
			{
				OutElements.Add(FSlateWidgetElementFactory::Create(WidgetPath));
			}
		}
	}


private:

	FSlateWidgetLocatorByWidgetDelegate(
		const FLocateSlateWidgetElementDelegate& InDelegate, FStringView InDebugName = { })
		: Delegate(InDelegate), DebugName(InDebugName)
	{ }

private:

	FLocateSlateWidgetElementDelegate Delegate;
	FString DebugName;

	friend FSlateWidgetLocatorByDelegateFactory;
};


TSharedRef<IElementLocator, ESPMode::ThreadSafe> FSlateWidgetLocatorByDelegateFactory::Create(
	const FLocateSlateWidgetElementDelegate& Delegate, FStringView DebugName)
{
	return MakeShareable(new FSlateWidgetLocatorByWidgetDelegate(Delegate, DebugName));
}

class FSlateWidgetLocatorByWidgetPathDelegate
	: public IElementLocator
{
public:

	virtual ~FSlateWidgetLocatorByWidgetPathDelegate()
	{ }

	virtual FString ToDebugString() const
	{
		FString DelegateName;
#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME
		DelegateName = Delegate.TryGetBoundFunctionName().ToString();
#endif
		return TEXT("[By::Delegate] ") + DelegateName;
	}

	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const override
	{
		check(IsInGameThread());

		TArray<FWidgetPath> WidgetPaths;
		Delegate.Execute(WidgetPaths);

		for (const FWidgetPath& WidgetPath : WidgetPaths)
		{
			if (WidgetPath.IsValid())
			{
				OutElements.Add(FSlateWidgetElementFactory::Create(WidgetPath));
			}
		}
	}


private:

	FSlateWidgetLocatorByWidgetPathDelegate(
		const FLocateSlateWidgetPathElementDelegate& InDelegate, FStringView InDebugName = { })
		: Delegate(InDelegate), DebugName(InDebugName)
	{ }

private:

	FLocateSlateWidgetPathElementDelegate Delegate;
	FString DebugName;

	friend FSlateWidgetLocatorByDelegateFactory;
};


TSharedRef<IElementLocator, ESPMode::ThreadSafe> FSlateWidgetLocatorByDelegateFactory::Create(
	const FLocateSlateWidgetPathElementDelegate& Delegate, FStringView DebugName)
{
	return MakeShareable(new FSlateWidgetLocatorByWidgetPathDelegate(Delegate, DebugName));
}
