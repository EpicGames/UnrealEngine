// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "LocateBy.h"

class IElementLocator;

class FSlateWidgetLocatorByDelegateFactory
{
public:

	static TSharedRef<IElementLocator, ESPMode::ThreadSafe> Create(
		const FLocateSlateWidgetElementDelegate& Delegate, FStringView DebugName = { });

	static TSharedRef<IElementLocator, ESPMode::ThreadSafe> Create(
		const FLocateSlateWidgetPathElementDelegate& Delegate, FStringView DebugName = { });
};
