// Copyright Epic Games, Inc. All Rights Reserved.

#include "Locators/SlateWidgetLocatorByPath.h"
#include "SlateWidgetElement.h"
#include "IElementLocator.h"
#include "Framework/MetaData/DriverIdMetaData.h"
#include "AutomationDriverLogging.h"
#include "Widgets/SWidget.h"
#include "Framework/Application/SlateApplication.h"
#include "AutomationDriverTypeDefs.h"
#include "IDriverElement.h"
#include "IApplicationElement.h"

class FSlateWidgetLocatorByPath
	: public IElementLocator
{
private:

	class FMatcher
	{
	public:

		virtual bool IsMatch(const TSharedRef<SWidget>& Widget) const = 0;

		bool bAllowRelativeDescendants;
	};

	class FIdMatcher
		: public FMatcher
	{
	public:

		virtual ~FIdMatcher()
		{ }

		virtual bool IsMatch(const TSharedRef<SWidget>& Widget) const override
		{
			const TArray<TSharedRef<FDriverIdMetaData>> AllIdMetaData = Widget->GetAllMetaData<FDriverIdMetaData>();

			bool bFoundMatch = false;

			for (int32 MetaDataIndex = 0; MetaDataIndex < AllIdMetaData.Num(); ++MetaDataIndex)
			{
				TSharedRef<FDriverIdMetaData> IdMetaData = AllIdMetaData[MetaDataIndex];

				if (IdMetaData->Id == Id)
				{
					bFoundMatch = true;
					break;
				}
			}

			return bFoundMatch;	
		}

		FIdMatcher(FName InId)
			: Id(MoveTemp(InId))
		{ }

		const FName Id;
	};

	class FTagMatcher
		: public FMatcher
	{
	public:

		virtual ~FTagMatcher()
		{ }

		virtual bool IsMatch(const TSharedRef<SWidget>& Widget) const override
		{
			if (Widget->GetTag() == Tag)
			{
				return true;
			}

			const TArray<TSharedRef<FTagMetaData>> AllMetaData = Widget->GetAllMetaData<FTagMetaData>();

			bool bFoundMatch = false;

			for (int32 MetaDataIndex = 0; MetaDataIndex < AllMetaData.Num(); ++MetaDataIndex)
			{
				TSharedRef<FTagMetaData> MetaData = AllMetaData[MetaDataIndex];

				if (MetaData->Tag == Tag)
				{
					bFoundMatch = true;
					break;
				}
			}

			return bFoundMatch;
		}

		FTagMatcher(FName InTag)
			: Tag(MoveTemp(InTag))
		{ }

		const FName Tag;
	};

	class FTypeMatcher
		: public FMatcher
	{
	public:

		virtual ~FTypeMatcher()
		{ }

		virtual bool IsMatch(const TSharedRef<SWidget>& Widget) const override
		{
			return Widget->GetType() == Type;
		}

		FTypeMatcher(FName InType)
			: Type(MoveTemp(InType))
		{ }

		const FName Type;
	};

public:

	virtual ~FSlateWidgetLocatorByPath()
	{ }

	virtual FString ToDebugString() const
	{
		return TEXT("[By::Path] ") + Path;
	}

	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const override
	{
		if (Matchers.Num() == 0)
		{
			return;
		}

		struct FStackState
		{
			FWidgetPath Path;
			int32 MatcherIndex;
		};
		TArray<FStackState> Stack;

		if (Root.IsValid())
		{
			TArray<TSharedRef<IApplicationElement>> OutRootElements;
			Root->Locate(OutRootElements);

			for (const TSharedRef<IApplicationElement>& RootElement : OutRootElements)
			{
				void* RawElementPtr = RootElement->GetRawElement();
				if (RawElementPtr == nullptr)
				{
					continue;
				}

				FWidgetPath* RootWidgetPath = static_cast<FWidgetPath*>(RawElementPtr);

				FStackState NewState;
				NewState.Path = *RootWidgetPath;
				NewState.MatcherIndex = 0;
				Stack.Push(NewState);
			}
		}
		else
		{
			// Get a list of all the current slate windows
			TArray<TSharedRef<SWindow>> Windows;
			FSlateApplication::Get().GetAllVisibleWindowsOrdered(/*OUT*/Windows);

			for (const TSharedRef<SWindow>& Window : Windows)
			{
				FStackState NewState;
				NewState.Path.TopLevelWindow = Window;
				NewState.Path.Widgets.AddWidget(FArrangedWidget(Window, Window->GetWindowGeometryInScreen()));
				NewState.MatcherIndex = 0;

				if (Matchers[NewState.MatcherIndex]->IsMatch(Window))
				{
					if (Matchers.IsValidIndex(NewState.MatcherIndex + 1))
					{
						NewState.MatcherIndex++;
						Stack.Push(NewState);
					}
					else
					{
						OutElements.Add(FSlateWidgetElementFactory::Create(NewState.Path));
					}
				}
				else
				{
					Stack.Push(NewState);
				}
			}
		}

		while (Stack.Num() > 0)
		{
			const FStackState State = Stack.Pop();
			const FArrangedWidget& Candidate = State.Path.Widgets.Last();

			const bool bAllow3DWidgets = true;
			const bool bUpdateVisibilityAttributes = true;
			FArrangedChildren ArrangedChildren(VisibilityFilter, bAllow3DWidgets);
			Candidate.Widget->ArrangeChildren(Candidate.Geometry, ArrangedChildren, bUpdateVisibilityAttributes);

			for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
			{
				const FArrangedWidget& SomeChild = ArrangedChildren[ChildIndex];

				if (Matchers[State.MatcherIndex]->IsMatch(SomeChild.Widget))
				{
					if (Matchers.IsValidIndex(State.MatcherIndex + 1))
					{
						FStackState NewState = State;
						NewState.Path.Widgets.AddWidget(SomeChild);
						NewState.MatcherIndex++;
						Stack.Push(NewState);
					}
					else
					{
						FWidgetPath NewPath = State.Path;
						NewPath.Widgets.AddWidget(SomeChild);
						OutElements.Add(FSlateWidgetElementFactory::Create(NewPath));
					}
				}
				else
				{
					if (Matchers.IsValidIndex(State.MatcherIndex - 1) && Matchers[State.MatcherIndex - 1]->bAllowRelativeDescendants)
					{
						FStackState NewState = State;
						NewState.Path.Widgets.AddWidget(SomeChild);
						Stack.Push(NewState);
					}
					else
					{
						FStackState NewState = State;
						NewState.Path.Widgets.AddWidget(SomeChild);
						NewState.MatcherIndex = 0;
						Stack.Push(NewState);
					}
				}
			}
		}
	}

private:

	FSlateWidgetLocatorByPath(
		const FDriverElementPtr& InRoot,
		const FString& InPath)
		: VisibilityFilter(EVisibility::Visible)
		, Root(InRoot)
		, Path(InPath)
	{
		FString TempPath = Path;

		if (!Path.IsEmpty())
		{
			bool bAllowRelativeDescendants = false;
			int32 Index;
			while (TempPath.FindChar(TEXT('/'), Index))
			{
				const FString PathPiece = TempPath.Left(Index);
				TempPath.RightChopInline(Index + 1, EAllowShrinking::No);

				if (PathPiece.Len() == 0)
				{
					if (Matchers.Last()->bAllowRelativeDescendants)
					{
						UE_LOGF(LogAutomationDriver, Error, "Invalid path specified as widget locator: %ls", *Path);
						Matchers.Empty();
						break;
					}
					else
					{
						Matchers.Last()->bAllowRelativeDescendants = true;
					}
				}
				else
				{
					AddMatcher(PathPiece);
					Matchers.Last()->bAllowRelativeDescendants = false;
				}
			}

			AddMatcher(TempPath);
			Matchers.Last()->bAllowRelativeDescendants = bAllowRelativeDescendants;
		}
	}

	void AddMatcher(const FString& PathPiece)
	{
		if (PathPiece[0] == TEXT('#'))
		{
			Matchers.Add(MakeShareable(new FIdMatcher(*PathPiece.Right(PathPiece.Len() - 1))));
		}
		else if (PathPiece[0] == TEXT('<'))
		{
			Matchers.Add(MakeShareable(new FTypeMatcher(*PathPiece.Mid(1, PathPiece.Len() - 2))));
		}
		else
		{
			Matchers.Add(MakeShareable(new FTagMatcher(*PathPiece)));
		}
	}

private:

	const EVisibility VisibilityFilter;
	const FDriverElementPtr Root;
	const FString Path;

	TArray<TSharedRef<FMatcher>> Matchers;

	friend FSlateWidgetLocatorByPathFactory;
};

TSharedRef<IElementLocator, ESPMode::ThreadSafe> FSlateWidgetLocatorByPathFactory::Create(
	const FString& Path)
{
	return FSlateWidgetLocatorByPathFactory::Create(nullptr, Path);
}

TSharedRef<IElementLocator, ESPMode::ThreadSafe> FSlateWidgetLocatorByPathFactory::Create(
	const FDriverElementPtr& Root,
	const FString& Path)
{
	return MakeShareable(new FSlateWidgetLocatorByPath(Root, Path));
}
