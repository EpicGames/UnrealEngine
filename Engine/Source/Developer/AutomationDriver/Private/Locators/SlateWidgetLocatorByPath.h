// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AutomationDriverTypeDefs.h"

class IElementLocator;

class FSlateWidgetLocatorByPathFactory
{
public:

	static TSharedRef<IElementLocator, ESPMode::ThreadSafe> Create(
		const FString& Path);

	static TSharedRef<IElementLocator, ESPMode::ThreadSafe> Create(
		const FDriverElementPtr& Root,
		const FString& Path);
};
