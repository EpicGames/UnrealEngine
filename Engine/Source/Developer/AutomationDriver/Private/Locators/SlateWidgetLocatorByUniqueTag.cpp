// Copyright Epic Games, Inc. All Rights Reserved.

#include "Locators/SlateWidgetLocatorByUniqueTag.h"
#include "SlateWidgetElement.h"
#include "IElementLocator.h"
#include "MetaData/DriverUniqueTagMetaData.h"
#include "Framework/Application/SlateApplication.h"

class FSlateWidgetLocatorByUniqueTag
	: public IElementLocator
{
public:

	virtual ~FSlateWidgetLocatorByUniqueTag()
	{ }

	virtual FString ToDebugString() const
	{
		const FString Address = FString::Printf(TEXT("0x%p"), &UniqueMetaData.Get());
		return TEXT("[UniqueTag] ") + Address;
	}

	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const override
	{
		// Get a list of all the current slate windows
		TArray<TSharedRef<SWindow>> Windows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(/*OUT*/Windows);

		struct FStackState
		{
			FWidgetPath Path;
		};
		TArray<FStackState> Stack;

		for (const TSharedRef<SWindow>& Window : Windows)
		{
			FStackState NewState;
			NewState.Path.Widgets.AddWidget(FArrangedWidget(Window, Window->GetWindowGeometryInScreen()));

			if (Window->GetAllMetaData<FDriverUniqueTagMetaData>().Contains(UniqueMetaData))
			{
				OutElements.Add(FSlateWidgetElementFactory::Create(NewState.Path));
				break;
			}
			else
			{
				Stack.Push(NewState);
			}
		}
		
		while (Stack.Num() > 0 && OutElements.Num() == 0)
		{
			const FStackState State = Stack.Pop();
			const FArrangedWidget& Candidate = State.Path.Widgets.Last();

			const bool bAllow3DWidgets = true;
			const bool bUpdateVisibilityAttributes = true;
			FArrangedChildren ArrangedChildren(VisibilityFilter, bAllow3DWidgets);
			Candidate.Widget->ArrangeChildren(Candidate.Geometry, ArrangedChildren, bUpdateVisibilityAttributes);

			for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
			{
				const FArrangedWidget& SomeChild = ArrangedChildren[ChildIndex];

				if (SomeChild.Widget->GetAllMetaData<FDriverUniqueTagMetaData>().Contains(UniqueMetaData))
				{
					FWidgetPath NewPath = State.Path;
					NewPath.Widgets.AddWidget(SomeChild);
					OutElements.Add(FSlateWidgetElementFactory::Create(NewPath));
					break;
				}
				else
				{
					FStackState NewState = State;
					NewState.Path.Widgets.AddWidget(SomeChild);
					Stack.Push(NewState);
				}
			}
		}
	}


private:

	FSlateWidgetLocatorByUniqueTag(
		const TSharedRef<FDriverUniqueTagMetaData>& InUniqueMetaData)
		: UniqueMetaData(InUniqueMetaData)
		, VisibilityFilter(EVisibility::Visible)
	{ }

private:

	const TSharedRef<FDriverUniqueTagMetaData> UniqueMetaData;
	const EVisibility VisibilityFilter;

	friend FSlateWidgetLocatorByUniqueTagFactory;
};


TSharedRef<IElementLocator, ESPMode::ThreadSafe> FSlateWidgetLocatorByUniqueTagFactory::Create(
	const TSharedRef<FDriverUniqueTagMetaData>& UniqueMetaData)
{
	return MakeShareable(new FSlateWidgetLocatorByUniqueTag(UniqueMetaData));
}
