// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IElementLocator;
class FDriverUniqueTagMetaData;

class FSlateWidgetLocatorByUniqueTagFactory
{
public:

	static TSharedRef<IElementLocator, ESPMode::ThreadSafe> Create(
		const TSharedRef<FDriverUniqueTagMetaData>& UniqueMetaData);
};
