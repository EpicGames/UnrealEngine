// Copyright Epic Games, Inc. All Rights Reserved.

#include "Locators/WidgetLocatorByFilter.h"
#include "IElementLocator.h"

class FWidgetLocatorByFilter
	: public IElementLocator
{
public:
	using FFilterFunction = FWidgetLocatorByFilterFactory::FFilterFunction;

	virtual ~FWidgetLocatorByFilter()
	{ }

	virtual FString ToDebugString() const
	{
		return RootLocator->ToDebugString() + " " + DebugString;
	}

	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const override
	{
		RootLocator->Locate(OutElements);

		for (int32 i = 0; i < OutElements.Num();)
		{
			if (Filter(OutElements[i]))
			{
				i++;
			}
			else
			{
				OutElements.RemoveAtSwap(i);
			}
		}
	}

private:
	FWidgetLocatorByFilter(
		const FString& InDebugString,
		const FElementLocatorRef& InRootLocator,
		FFilterFunction InFilter) :
		DebugString{ InDebugString },
		RootLocator{ InRootLocator },
		Filter{ MoveTemp(InFilter) }
	{ }

private:

	const FString DebugString;
	const FElementLocatorRef RootLocator;
	const FFilterFunction Filter;

	friend FWidgetLocatorByFilterFactory;
};

TSharedRef<IElementLocator, ESPMode::ThreadSafe> FWidgetLocatorByFilterFactory::Create(
	const FString& DebugString,
	const FElementLocatorRef& RootLocator,
	FFilterFunction Filter)
{
	return MakeShareable(new FWidgetLocatorByFilter(DebugString, RootLocator, MoveTemp(Filter)));
}
