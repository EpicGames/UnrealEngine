// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AutomationDriverTypeDefs.h"

class IApplicationElement;
class IElementLocator;

/* This class provides a utility function for creating instances of the filter locator. */
class FWidgetLocatorByFilterFactory
{
public:
	using FFilterFunction = TFunction<bool(const TSharedRef<IApplicationElement>&)>;
	
	/**
	* Creates a new element locator that limits the elements discovered by the provided root locator to those determined by the filter function.
	*
	* @param DebugString - The string used for debugging and logging purposes to represent this particular instance of the filter locator
	* @param RootLocator - The reference to the element locator where the search will be started from
	* @param Filter - The function applied to the elements found by the root locator to determine which should be saved as a result.
	* @return a locator which uses the specified root locator and a filter function to discover appropriate elements
	*/
	static TSharedRef<IElementLocator, ESPMode::ThreadSafe> Create(
		const FString& DebugString,
		const FElementLocatorRef& RootLocator,
		FFilterFunction Filter
	);
};
