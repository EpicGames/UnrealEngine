// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Types/ISlateMetaData.h"

class FDriverUniqueTagMetaData
	: public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FDriverUniqueTagMetaData, ISlateMetaData)

	FDriverUniqueTagMetaData()
	{ }
};
