// Copyright Epic Games, Inc. All Rights Reserved.

#include "PassThroughMessageHandler.h"
#include "InputCoreTypes.h"

class FPassThroughMessageHandlerFactory;

class FPassThroughMessageHandlerImpl
	: public FPassThroughMessageHandler
{
public:

	virtual ~FPassThroughMessageHandlerImpl() override
	{ }

	virtual bool IsHandlingMessages() const override
	{
		return bAllowMessageHandling;
	}

	virtual void SetAllowMessageHandling(bool bValue) override
	{
		bAllowMessageHandling = bValue;
	}

public:

	virtual bool ShouldProcessUserInputMessages(const TSharedPtr< FGenericWindow >& PlatformWindow) const override
	{
		return RealMessageHandler->ShouldProcessUserInputMessages(PlatformWindow);
	}

	virtual bool OnKeyChar(const TCHAR Character, const bool IsRepeat) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnKeyChar(Character, IsRepeat);
	}

	virtual bool OnKeyDown(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnKeyDown(KeyCode, CharacterCode, IsRepeat);
	}

	virtual bool OnKeyUp(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat) override
	{
		const FKey Key = FInputKeyManager::Get().GetKeyFromCodes(KeyCode, CharacterCode);

		if (Key == EKeys::ScrollLock)
		{
			// Allow scroll lock to toggle whether platform input can be processed by the application
			bAllowMessageHandling = !bAllowMessageHandling;
		}

		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnKeyUp(KeyCode, CharacterCode, IsRepeat);
	}

	virtual bool OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMouseDown(Window, Button);
	}

	virtual bool OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMouseDown(Window, Button, CursorPos);
	}

	virtual bool OnMouseUp(const EMouseButtons::Type Button) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMouseUp(Button);
	}

	virtual bool OnMouseUp(const EMouseButtons::Type Button, const FVector2D CursorPos) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMouseUp(Button, CursorPos);
	}

	virtual bool OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMouseDoubleClick(Window, Button);
	}

	virtual bool OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMouseDoubleClick(Window, Button, CursorPos);
	}

	virtual bool OnMouseWheel(const float Delta) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMouseWheel(Delta);
	}

	virtual bool OnMouseWheel(const float Delta, const FVector2D CursorPos) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMouseWheel(Delta, CursorPos);
	}

	virtual bool OnMouseMove() override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMouseMove();
	}

	virtual bool OnRawMouseMove(const int32 X, const int32 Y) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnRawMouseMove(X, Y);
	}

	virtual bool OnCursorSet() override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnCursorSet();
	}
	
	virtual bool OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnControllerAnalog(KeyName, PlatformUserId, InputDeviceId, AnalogValue);
	}

	virtual bool OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnControllerButtonPressed(KeyName, PlatformUserId, InputDeviceId, IsRepeat);
	}

	virtual bool OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnControllerButtonReleased(KeyName, PlatformUserId, InputDeviceId, IsRepeat);
	}

	virtual void OnBeginGesture() override
	{
		if (!bAllowMessageHandling)
		{
			return;
		}

		return RealMessageHandler->OnBeginGesture();
	}

	virtual bool OnTouchGesture(EGestureEvent GestureType, const FVector2D& Delta, float WheelDelta, bool bIsDirectionInvertedFromDevice) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnTouchGesture(GestureType, Delta, WheelDelta, bIsDirectionInvertedFromDevice);
	}

	virtual void OnEndGesture() override
	{
		if (!bAllowMessageHandling)
		{
			return;
		}

		RealMessageHandler->OnEndGesture();
	}

	virtual bool OnTouchStarted(const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnTouchStarted(Window, Location, Force, TouchIndex, ControllerId);
	}

	virtual bool OnTouchMoved(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnTouchMoved(Location, Force, TouchIndex, ControllerId);
	}

	virtual bool OnTouchEnded(const FVector2D& Location, int32 TouchIndex, int32 ControllerId) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnTouchEnded(Location, TouchIndex, ControllerId);
	}

	virtual bool OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnTouchForceChanged(Location, Force, TouchIndex, ControllerId);
	}

	virtual bool OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnTouchFirstMove(Location, Force, TouchIndex, ControllerId);
	}

	virtual bool OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, int32 ControllerId) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnMotionDetected(Tilt, RotationRate, Gravity, Acceleration, ControllerId);
	}

	virtual bool OnSizeChanged(const TSharedRef< FGenericWindow >& Window, const int32 Width, const int32 Height, bool bWasMinimized = false) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->OnSizeChanged(Window, Width, Height, bWasMinimized);
	}

	virtual void OnOSPaint(const TSharedRef<FGenericWindow>& Window) override
	{
		RealMessageHandler->OnOSPaint(Window);
	}

	virtual FWindowSizeLimits GetSizeLimitsForWindow(const TSharedRef<FGenericWindow>& Window) const override
	{
		return RealMessageHandler->GetSizeLimitsForWindow(Window);
	}

	virtual void OnResizingWindow(const TSharedRef<FGenericWindow>& Window) override
	{
		if (!bAllowMessageHandling)
		{
			return;
		}

		RealMessageHandler->OnResizingWindow(Window);
	}

	virtual bool BeginReshapingWindow(const TSharedRef< FGenericWindow >& Window) override
	{
		if (!bAllowMessageHandling)
		{
			return false;
		}

		return RealMessageHandler->BeginReshapingWindow(Window);
	}

	virtual void FinishedReshapingWindow(const TSharedRef< FGenericWindow >& Window) override
	{
		if (!bAllowMessageHandling)
		{
			return;
		}

		RealMessageHandler->FinishedReshapingWindow(Window);
	}

	virtual void OnMovedWindow(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y) override
	{
		if (!bAllowMessageHandling)
		{
			return;
		}

		RealMessageHandler->OnMovedWindow(Window, X, Y);
	}

	virtual bool OnWindowActivationChanged(const TSharedRef< FGenericWindow >& Window, const EWindowActivation ActivationType) override
	{
		return RealMessageHandler->OnWindowActivationChanged(Window, ActivationType);
	}

	virtual bool OnApplicationActivationChanged(const bool IsActive) override
	{
		return RealMessageHandler->OnApplicationActivationChanged(IsActive);
	}

	virtual bool OnConvertibleLaptopModeChanged() override
	{
		if (!bAllowMessageHandling)
		{
			return true;
		}

		return RealMessageHandler->OnConvertibleLaptopModeChanged();
	}

	virtual EWindowZone::Type GetWindowZoneForPoint(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y) override
	{
		if (!bAllowMessageHandling)
		{
			return EWindowZone::NotInWindow;
		}

		return RealMessageHandler->GetWindowZoneForPoint(Window, X, Y);
	}

	virtual void OnWindowClose(const TSharedRef< FGenericWindow >& Window) override
	{
		if (!bAllowMessageHandling)
		{
			return;
		}

		RealMessageHandler->OnWindowClose(Window);
	}

	virtual EDropEffect::Type OnDragEnterText(const TSharedRef< FGenericWindow >& Window, const FString& Text) override
	{
		if (!bAllowMessageHandling)
		{
			return EDropEffect::None;
		}

		return RealMessageHandler->OnDragEnterText(Window, Text);
	}

	virtual EDropEffect::Type OnDragEnterFiles(const TSharedRef< FGenericWindow >& Window, const TArray< FString >& Files) override
	{
		if (!bAllowMessageHandling)
		{
			return EDropEffect::None;
		}

		return RealMessageHandler->OnDragEnterFiles(Window, Files);
	}

	virtual EDropEffect::Type OnDragOver(const TSharedPtr< FGenericWindow >& Window) override
	{
		if (!bAllowMessageHandling)
		{
			return EDropEffect::None;
		}

		return RealMessageHandler->OnDragOver(Window);
	}

	virtual void OnDragLeave(const TSharedPtr< FGenericWindow >& Window) override
	{
		if (!bAllowMessageHandling)
		{
			return;
		}

		RealMessageHandler->OnDragLeave(Window);
	}

	virtual EDropEffect::Type OnDragDrop(const TSharedPtr< FGenericWindow >& Window) override
	{
		if (!bAllowMessageHandling)
		{
			return EDropEffect::None;
		}

		return RealMessageHandler->OnDragDrop(Window);
	}

	virtual bool OnWindowAction(const TSharedRef< FGenericWindow >& Window, const EWindowAction::Type InActionType) override
	{
		if (!bAllowMessageHandling)
		{
			return true;
		}

		return RealMessageHandler->OnWindowAction(Window, InActionType);
	}

private:

	FPassThroughMessageHandlerImpl(
		const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
		: RealMessageHandler(InMessageHandler)
		, bAllowMessageHandling(false)
	{ }

private:

	const TSharedRef<FGenericApplicationMessageHandler> RealMessageHandler;

	bool bAllowMessageHandling;

	friend FPassThroughMessageHandlerFactory;
};

class FPassThroughMessageHandlerFactory
	: public IPassThroughMessageHandlerFactory
{
public:

	virtual ~FPassThroughMessageHandlerFactory() override
	{ }

	virtual TSharedRef<FPassThroughMessageHandler> Create(
		const TSharedRef<FGenericApplicationMessageHandler>& MessageHandler) const override
	{
		return MakeShareable(new FPassThroughMessageHandlerImpl(MessageHandler));
	}
};

TSharedRef<IPassThroughMessageHandlerFactory> FPassThroughMessageHandlerFactoryFactory::Create()
{
	return MakeShareable(new FPassThroughMessageHandlerFactory());
}
