// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GenericPlatform/GenericApplicationMessageHandler.h"

class FPassThroughMessageHandler
	: public FGenericApplicationMessageHandler
{
public:

	virtual ~FPassThroughMessageHandler() = default;

	virtual bool IsHandlingMessages() const = 0;

	virtual void SetAllowMessageHandling(bool bValue) = 0;

};

class IPassThroughMessageHandlerFactory
{
public:

	virtual ~IPassThroughMessageHandlerFactory() = default;

	virtual TSharedRef<FPassThroughMessageHandler> Create(
		const TSharedRef<FGenericApplicationMessageHandler>& MessageHandler) const = 0;
};

class FPassThroughMessageHandlerFactoryFactory
{
public:

	static TSharedRef<IPassThroughMessageHandlerFactory> Create();
};
