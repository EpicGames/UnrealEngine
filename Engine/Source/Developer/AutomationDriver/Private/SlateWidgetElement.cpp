// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateWidgetElement.h"
#include "IApplicationElement.h"
#include "Framework/Application/SlateApplication.h"

#include "Locators/SlateWidgetLocatorByUniqueTag.h"
#include "MetaData/DriverUniqueTagMetaData.h"
#include "Framework/MetaData/DriverIdMetaData.h"

#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "UObject/NameTypes.h"

class FSlateWidgetElement
	: public IApplicationElement
{
public:

	virtual ~FSlateWidgetElement()
	{ }

	virtual FString ToDebugString() const
	{
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;
		const TArray<TSharedRef<FDriverIdMetaData>> AllIdMetaData = Widget->GetAllMetaData<FDriverIdMetaData>();
		const TArray<TSharedRef<FTagMetaData>> AllTagMetaData = Widget->GetAllMetaData<FTagMetaData>();

		//example (STextBlock) 0x00082034 [#Piano|Keyboard] : SlateWidgetElement.cpp(20)
		FString DebugString;

		const FString Address = FString::Printf(TEXT("0x%p"), &Widget.Get());
		DebugString += TEXT("(") + Widget->GetTypeAsString() + TEXT(") ") + Address;

		if (AllIdMetaData.Num() + AllTagMetaData.Num() > 0 || !Widget->GetTag().IsNone())
		{
			DebugString += TEXT(" [");

			bool bHasPrecedingIdentifier = false;
			for (const TSharedRef<FDriverIdMetaData>& MetaData : AllIdMetaData)
			{
				if (bHasPrecedingIdentifier)
				{
					DebugString += TEXT("|");
				}

				DebugString += TEXT("#");
				DebugString += MetaData->Id.ToString();
				bHasPrecedingIdentifier = true;
			}

			if (!Widget->GetTag().IsNone())
			{
				if (bHasPrecedingIdentifier)
				{
					DebugString += TEXT("|");
				}

				DebugString += Widget->GetTag().ToString();
			}

			for (const TSharedRef<FTagMetaData>& MetaData : AllTagMetaData)
			{
				if (bHasPrecedingIdentifier)
				{
					DebugString += TEXT("|");
				}

				DebugString += MetaData->Tag.ToString();
				bHasPrecedingIdentifier = true;
			}
			DebugString += TEXT("]");
		}
		
		DebugString += TEXT(" : ") + Widget->GetReadableLocation();

		return DebugString;
	}

	virtual FVector2D GetAbsolutePosition() const override
	{
		return WidgetPath.Widgets.Last().Geometry.LocalToAbsolute(FVector2D::ZeroVector);
	}

	virtual FVector2D GetSize() const override
	{
		return WidgetPath.Widgets.Last().Geometry.GetDrawSize();
	}

	virtual TSharedPtr<FGenericWindow> GetWindow() const override
	{
		const TSharedPtr<SWindow> Window = WidgetPath.GetWindow();
		if (!Window.IsValid())
		{
			return nullptr;
		}

		return Window->GetNativeWindow();
	}

	virtual bool IsVisible() const override
	{
		static FName SWindowType(TEXT("SWindow"));
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;

		if (Widget->GetVisibility().IsVisible())
		{
			FVector2D FakeCursorPosition = GetAbsolutePosition() + (GetSize() / 2);

			TArray<TSharedRef<SWindow>> Windows;
			if (WidgetPath.TopLevelWindow.IsValid())
			{
				Windows.Add(WidgetPath.TopLevelWindow.ToSharedRef());
			}
			else if (WidgetPath.Widgets[0].Widget->GetType() == SWindowType)
			{
				Windows.Add(StaticCastSharedRef<SWindow>(WidgetPath.Widgets[0].Widget));
			}
			else
			{
				return false;
			}

			FWidgetPath UnderCursor = FSlateApplication::Get().LocateWindowUnderMouse(FakeCursorPosition, Windows, true);

			int32 Count = UnderCursor.Widgets.Num();
			for (int32 Index = 0; Index < Count; Index++)
			{
				TSharedRef<SWidget> FoundWidget = UnderCursor.Widgets[Index].Widget;
				if (FoundWidget == Widget)
				{
					return true;
				}
			}
		}

		return false;
	}

	virtual bool IsInteractable() const override
	{
		static FName SWindowType(TEXT("SWindow"));
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;

		if (Widget->IsEnabled() && Widget->GetVisibility().IsHitTestVisible())
		{
			FVector2D FakeCursorPosition = GetAbsolutePosition() + (GetSize() / 2);

			TArray<TSharedRef<SWindow>> Windows;
			if (WidgetPath.TopLevelWindow.IsValid())
			{
				Windows.Add(WidgetPath.TopLevelWindow.ToSharedRef());
			}
			else if (WidgetPath.Widgets[0].Widget->GetType() == SWindowType)
			{
				Windows.Add(StaticCastSharedRef<SWindow>(WidgetPath.Widgets[0].Widget));
			}
			else
			{
				return false;
			}

			FWidgetPath UnderCursor = FSlateApplication::Get().LocateWindowUnderMouse(FakeCursorPosition, Windows, false);

			int32 Count = UnderCursor.Widgets.Num();
			for (int32 Index = 0; Index < Count; Index++)
			{
				TSharedRef<SWidget> FoundWidget = UnderCursor.Widgets[Index].Widget;
				if (FoundWidget == Widget)
				{
					return true;
				}
			}
		}

		return false;
	}

	virtual bool IsChecked() const override
	{
		static FName SCheckBoxType(TEXT("SCheckBox"));

		const TSharedRef<SWidget> RootWidget = WidgetPath.Widgets.Last().Widget;

		TArray<TSharedRef<SWidget>> Stack;
		Stack.Push(RootWidget);

		bool bFoundCheck = false;
		bool bResult = false;

		while (Stack.Num() > 0)
		{
			const TSharedRef<SWidget> Widget = Stack.Pop();

			if (Widget->GetType() == SCheckBoxType)
			{
				if (bFoundCheck == true)
				{
					return false;
				}

				bResult = StaticCastSharedRef<SCheckBox, SWidget>(Widget)->IsChecked();
				bFoundCheck = true;
			}
			else
			{
				FChildren* ChildWidgets = Widget->GetChildren();
				int32 ChildCount = ChildWidgets->Num();

				for (int32 i = 0; i < ChildCount; i++)
				{
					Stack.Push(ChildWidgets->GetChildAt(i));
				}
			}
		}

		return bResult;
	}

	virtual FText GetText() const override
	{
		static FName SRichTextBlockType(TEXT("SRichTextBlock"));
		static FName STextBlockType(TEXT("STextBlock"));
		static FName SEditableTextType(TEXT("SEditableText"));
		static FName SEditableTextBoxType(TEXT("SEditableTextBox"));
		static FName SMultiLineEditableTextType(TEXT("SMultiLineEditableText"));
		static FName SMultiLineEditableTextBoxType(TEXT("SMultiLineEditableTextBox"));

		const TSharedRef<SWidget> RootWidget = WidgetPath.Widgets.Last().Widget;

		TArray<TSharedRef<SWidget>> Stack;
		Stack.Push(RootWidget);

		TArray<FText> WidgetsText;

		while(Stack.Num() > 0)
		{
			const TSharedRef<SWidget> Widget = Stack.Pop();
			FText WidgetText = FText::GetEmpty();

			if (Widget->GetType() == STextBlockType)
			{
				WidgetText = StaticCastSharedRef<STextBlock, SWidget>(Widget)->GetText();
			}
			else if (Widget->GetType() == SEditableTextType)
			{
				WidgetText = StaticCastSharedRef<SEditableText, SWidget>(Widget)->GetText();
			}
			else if (Widget->GetType() == SEditableTextBoxType)
			{
				WidgetText = StaticCastSharedRef<SEditableTextBox, SWidget>(Widget)->GetText();
			}
			else if (Widget->GetType() == SRichTextBlockType)
			{
				WidgetText = StaticCastSharedRef<SRichTextBlock, SWidget>(Widget)->GetText();
			}
			else if (Widget->GetType() == SMultiLineEditableTextType)
			{
				WidgetText = StaticCastSharedRef<SMultiLineEditableText, SWidget>(Widget)->GetText();
			}
			else if (Widget->GetType() == SMultiLineEditableTextBoxType)
			{
				WidgetText = StaticCastSharedRef<SMultiLineEditableTextBox, SWidget>(Widget)->GetText();
			}
			else
			{
				FChildren* ChildWidgets = Widget->GetChildren();
				int32 ChildCount = ChildWidgets->Num();

				for (int32 i = 0; i < ChildCount; i++)
				{
					Stack.Push(ChildWidgets->GetChildAt(i));
				}
			}

			if (!WidgetText.IsEmpty())
			{
				WidgetsText.Add(WidgetText);
			}
		}

		return FText::Join(FText::FromString(LINE_TERMINATOR), WidgetsText);
	}

	virtual TSharedRef<IElementLocator, ESPMode::ThreadSafe> CreateLocator() const override
	{
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;

		const TSharedRef<FDriverUniqueTagMetaData> UniqueMetaData = MakeShareable(new FDriverUniqueTagMetaData());
		Widget->AddMetadata(UniqueMetaData);

		return FSlateWidgetLocatorByUniqueTagFactory::Create(UniqueMetaData);
	}

	virtual bool CanFocus() const override
	{
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;
		return Widget->SupportsKeyboardFocus();
	}

	virtual bool Focus() const override
	{
		return FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), WidgetPath, EFocusCause::SetDirectly);
	}

	virtual bool Focus(uint32 UserIndex) const override
	{
		return FSlateApplication::Get().SetUserFocus(UserIndex, WidgetPath, EFocusCause::SetDirectly);
	}

	virtual bool IsFocused() const override
	{
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;
		return FSlateApplication::Get().GetUserFocusedWidget(FSlateApplication::Get().GetUserIndexForKeyboard()) == Widget;
	}

	virtual bool IsFocused(uint32 UserIndex) const override
	{
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;
		return FSlateApplication::Get().GetUserFocusedWidget(UserIndex) == Widget;
	}

	virtual bool HasFocusedDescendants() const override
	{
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;
		return FSlateApplication::Get().HasFocusedDescendants(Widget);
	}

	virtual bool HasFocusedDescendants(uint32 UserIndex) const override
	{
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;
		return FSlateApplication::Get().HasUserFocusedDescendants(Widget, UserIndex);
	}

	virtual bool IsHovered() const override
	{
		const TSharedRef<SWidget> Widget = WidgetPath.Widgets.Last().Widget;
		return Widget->IsHovered();
	}

	static TSharedPtr<SScrollBar> HasScrollableDescendants(const FWidgetPath& WidgetPath, int32& ErrorCode)
	{
		TArray<TSharedRef<SWidget>> IgnoreWidgets;
		return HasScrollableDescendants(WidgetPath, IgnoreWidgets, ErrorCode);
	}

	static TSharedPtr<SScrollBar> HasScrollableDescendants(const FWidgetPath& WidgetPath, const TArray<TSharedRef<SWidget>>& IgnoreWidgets, int32& ErrorCode)
	{
		static FName SScrollBarType(TEXT("SScrollBar"));

		TArray<TSharedRef<SWidget>> Stack;
		Stack.Push(WidgetPath.Widgets.Last().Widget);

		TSharedPtr<SScrollBar> ScrollBar;

		while (Stack.Num() > 0)
		{
			const TSharedRef<SWidget> Widget = Stack.Pop();

			if (Widget->GetType() == SScrollBarType)
			{
				TSharedPtr<SScrollBar> FoundScrollBar = StaticCastSharedRef<SScrollBar, SWidget>(Widget);
				
				if (FoundScrollBar->IsNeeded())
				{
					if (ScrollBar.IsValid() && ScrollBar != FoundScrollBar)
					{
						ErrorCode = 2;
						return nullptr;
					}

					ScrollBar = FoundScrollBar;
				}
			}
			else
			{
				FChildren* ChildWidgets = Widget->GetChildren();
				int32 ChildCount = ChildWidgets->Num();

				for (int32 i = 0; i < ChildCount; i++)
				{
					const TSharedRef<SWidget> ChildWidget = ChildWidgets->GetChildAt(i);

					if (!IgnoreWidgets.Contains(ChildWidget))
					{
						Stack.Push(ChildWidget);
					}
				}
			}
		}

		if (!ScrollBar.IsValid())
		{
			ErrorCode = 1;
			return nullptr;
		}

		ErrorCode = 0;
		return ScrollBar;
	}

	virtual bool IsScrollable() const override
	{
		EOrientation ScrollOrientation;
		return IsScrollable(ScrollOrientation);
	}

	virtual bool IsScrollable(EOrientation& OutScrollOrientation) const override
	{
		int32 ErrorCode;
		const TSharedPtr<SScrollBar> ScrollBar = HasScrollableDescendants(WidgetPath, ErrorCode);

		if (ScrollBar.IsValid())
		{
			OutScrollOrientation = ScrollBar->GetOrientation();
		}

		return ScrollBar.IsValid();
	}

	virtual bool IsScrolledToBeginning() const override
	{
		int32 ErrorCode;
		const TSharedPtr<SScrollBar> ScrollBar = HasScrollableDescendants(WidgetPath, ErrorCode);

		if (!ScrollBar.IsValid())
		{
			return true;
		}

		return ScrollBar->DistanceFromTop() < KINDA_SMALL_NUMBER;
	}

	virtual bool IsScrolledToEnd() const override
	{
		int32 ErrorCode;
		const TSharedPtr<SScrollBar> ScrollBar = HasScrollableDescendants(WidgetPath, ErrorCode);

		if (!ScrollBar.IsValid())
		{
			return true;
		}

		return ScrollBar->DistanceFromBottom() < KINDA_SMALL_NUMBER;
	}

	virtual TSharedPtr<IApplicationElement> GetScrollableParent() const override
	{
		FWidgetPath ParentWidgetPath = WidgetPath;
		ParentWidgetPath.Widgets.Remove(ParentWidgetPath.Widgets.Num() - 1);

		TArray<TSharedRef<SWidget>> IgnoreWidgets;
		IgnoreWidgets.Add(WidgetPath.Widgets.Last().Widget);

		bool bFoundScrollableParent = false;
		for (int Index = WidgetPath.Widgets.Num() - 2; Index >= 0; Index--)
		{
			const TSharedRef<SWidget> ParentWidget = WidgetPath.Widgets[Index].Widget;

			int32 ErrorCode;
			const TSharedPtr<SScrollBar> ScrollBar = HasScrollableDescendants(ParentWidgetPath, IgnoreWidgets, ErrorCode);

			if (ScrollBar.IsValid())
			{
				bFoundScrollableParent = true;
				break;
			}
			else if (ErrorCode == 2)
			{
				bFoundScrollableParent = false;
				break;
			}

			IgnoreWidgets.Add(ParentWidget);
			ParentWidgetPath.Widgets.Remove(ParentWidgetPath.Widgets.Num() - 1);
		}

		if (bFoundScrollableParent)
		{
			return FSlateWidgetElementFactory::Create(ParentWidgetPath);
		}

		return nullptr;
	}

	virtual void* GetRawElement() const override
	{
		if (!WidgetPath.IsValid())
		{
			return nullptr;
		}

		return (void*)(&WidgetPath);
	}

private:

	FSlateWidgetElement(
		const FWidgetPath& InWidgetPath)
		: WidgetPath(InWidgetPath)
	{ }

private:

	const FWidgetPath WidgetPath;

	friend FSlateWidgetElementFactory;
};

TSharedRef<IApplicationElement> FSlateWidgetElementFactory::Create(
	const FWidgetPath& WidgetPath)
{
	return MakeShareable(new FSlateWidgetElement(WidgetPath));
}
