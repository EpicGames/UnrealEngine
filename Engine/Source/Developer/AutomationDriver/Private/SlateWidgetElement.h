// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/WidgetPath.h"

class IApplicationElement;

class FSlateWidgetElementFactory
{
public:

	static TSharedRef<IApplicationElement> Create(
		const FWidgetPath& WidgetPath);
};