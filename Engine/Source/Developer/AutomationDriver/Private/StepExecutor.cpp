// Copyright Epic Games, Inc. All Rights Reserved.

#include "StepExecutor.h"
#include "IStepExecutor.h"
#include "IAutomationDriver.h"
#include "AutomatedApplication.h"

#include "DriverConfiguration.h"

#include "Misc/ScopeLock.h"
#include "Containers/Ticker.h"
#include "Async/Async.h"


class FStepExecutor
	: public IStepExecutor
	, public TSharedFromThis<FStepExecutor, ESPMode::ThreadSafe>
{
public:

	virtual ~FStepExecutor()
	{
		if (Promise.IsValid())
		{
			Promise->SetValue(false);
		}

		// As we use guards to access to the Steps array we have to clear it explicitly to sync access to it
		FScopeLock StateLock(&StepsCS);
		Steps.Empty();
	}

	virtual void Add(const TSharedRef<FExecuteStepDelegate>& Step) override
	{
		check(!Promise.IsValid());
		FScopeLock StateLock(&StepsCS);
		Steps.Add(Step);
	}

	virtual void Add(const TFunction<FStepResult(const FTimespan&)>& StepFunction) override
	{
		check(StepFunction);
		TSharedRef<FExecuteStepDelegate> Step = MakeShared<FExecuteStepDelegate>(
			FExecuteStepDelegate::CreateLambda(StepFunction));
		Add(Step);
	}

	virtual void InsertNext(const TSharedRef<FExecuteStepDelegate>& Step) override
	{
		check(Promise.IsValid());
		FScopeLock StateLock(&StepsCS);
		Steps.Insert(Step, CurrentStepIndex + 1);
	}

	virtual void InsertNext(const TFunction<FStepResult(const FTimespan&)>& StepFunction) override
	{
		check(StepFunction);
		TSharedRef<FExecuteStepDelegate> Step = MakeShared<FExecuteStepDelegate>(
			FExecuteStepDelegate::CreateLambda(StepFunction));
		InsertNext(Step);
	}

	virtual TAsyncResult<bool> Execute() override
	{
		check(!Promise.IsValid());
		CurrentStepIndex = 0;

		// Queue result back on main thread
		TWeakPtr<FStepExecutor, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));
		AsyncTask(
			ENamedThreads::GameThread,
			[LocalWeakThis]()
			{
				TSharedPtr<FStepExecutor, ESPMode::ThreadSafe> Executor = LocalWeakThis.Pin();

				if (Executor.IsValid())
				{
					const int32 StepIndex = 0;
					FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateThreadSafeSP(Executor.ToSharedRef(), &FStepExecutor::ExecuteStep, StepIndex), 0);
				}
			}
		);

		Promise = MakeShareable(new TPromise<bool>());
		return TAsyncResult<bool>(Promise->GetFuture(), nullptr, nullptr);
	}

	virtual bool IsExecuting() const override
	{
		return Promise.IsValid();
	}
	
	virtual void OnCompleted(TFunction<void()> Callback) override
	{
		OnCompletedCallback = MoveTemp(Callback);
	}

private:

	FStepExecutor(
		const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& InConfiguration,
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& InApplication)
		: Configuration(InConfiguration)
		, Application(InApplication)
		, Steps()
		, CurrentStepIndex(0)
		, Promise()
		, StepTotalProcessTime(FTimespan::Zero())
		, LastDelay(0)
	{ }

	bool ExecuteStep(float Delta, int32 StepIndex)
	{
        QUICK_SCOPE_CYCLE_COUNTER(STAT_FStepExecutor_ExecuteStep);

		check(IsInGameThread());

		FStepResult Result = FStepResult(FStepResult::EState::FAILED, 0);
		{
			FScopeLock StateLock(&StepsCS);

			if (0 == StepIndex)
			{
				Application->SetOverrideRealCursorCoordinates(true);
			}

			// If we've encountered an invalid step that's greater then zero then we were just waiting
			// a little bit after the last step completed before signaling completion.
			if ((StepIndex > 0 && !Steps.IsValidIndex(StepIndex)) || !Application->IsHandlingMessages())
			{
				Application->SetOverrideRealCursorCoordinates(false); 
				Promise->SetValue(true);
				Promise.Reset();
				StepTotalProcessTime = FTimespan::Zero();
				if (OnCompletedCallback) { OnCompletedCallback(); }
				return false;
			}

			check(Steps.IsValidIndex(StepIndex));

			Result = Steps[StepIndex]->Execute(StepTotalProcessTime);
		}

		if (Result.State == FStepResult::EState::FAILED)
		{
			Application->SetOverrideRealCursorCoordinates(false);
			Promise->SetValue(false);
			Promise.Reset();
			StepTotalProcessTime = FTimespan::Zero();
			if (OnCompletedCallback) { OnCompletedCallback(); }
			return false;
		}

		if (Result.State == FStepResult::EState::DONE)
		{
			StepTotalProcessTime = FTimespan::Zero();
			++StepIndex;
		}

		CurrentStepIndex = StepIndex;
		float Milliseconds = (float)(Result.NextWait.GetTicks()) / ETimespan::TicksPerMillisecond;
		float Delay = FMath::Max(SMALL_NUMBER, (Milliseconds / 1000) * Configuration->ExecutionSpeedMultiplier);
		
		if (LastDelay < KINDA_SMALL_NUMBER)
		{
			StepTotalProcessTime += FTimespan::FromSeconds(Delta);
		}

		StepTotalProcessTime += FTimespan::FromSeconds(Delay);
		LastDelay = Delay;
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateThreadSafeSP(this, &FStepExecutor::ExecuteStep, StepIndex), Delay);

		return false;
	}

private:

	const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe> Configuration;
	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& Application;

	TArray<TSharedRef<FExecuteStepDelegate>> Steps;
	int32 CurrentStepIndex;
	TSharedPtr<TPromise<bool>> Promise;
	FTimespan StepTotalProcessTime;
	float LastDelay;
	TFunction<void()> OnCompletedCallback;

	mutable FCriticalSection StepsCS;

	friend FStepExecutorFactory;
};

TSharedRef<IStepExecutor, ESPMode::ThreadSafe> FStepExecutorFactory::Create(
	const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration,
	const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& Application)
{
	return MakeShareable(new FStepExecutor(Configuration, Application));
}
