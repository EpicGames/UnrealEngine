// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IStepExecutor;
class FDriverConfiguration;
class FAutomatedApplication;

class FStepExecutorFactory
{
public:

	static TSharedRef<IStepExecutor, ESPMode::ThreadSafe> Create(
		const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration,
		const TSharedRef<FAutomatedApplication, ESPMode::ThreadSafe>& Application);

};
