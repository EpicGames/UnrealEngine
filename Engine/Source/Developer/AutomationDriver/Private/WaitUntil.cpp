// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaitUntil.h"
#include "IApplicationElement.h"
#include "IElementLocator.h"
#include "Misc/Timespan.h"

FDriverWaitResponse FDriverWaitResponse::Passed()
{
	return FDriverWaitResponse(FDriverWaitResponse::EState::PASSED, FTimespan::Zero());
}

FDriverWaitResponse FDriverWaitResponse::Wait()
{
	return FDriverWaitResponse(FDriverWaitResponse::EState::WAIT, FTimespan::FromSeconds(0.5));
}

FDriverWaitResponse FDriverWaitResponse::Wait(FTimespan Timespan)
{
	return FDriverWaitResponse(FDriverWaitResponse::EState::WAIT, MoveTemp(Timespan));
}

FDriverWaitResponse FDriverWaitResponse::Failed()
{
	return FDriverWaitResponse(FDriverWaitResponse::EState::FAILED, FTimespan::Zero());
}

FDriverWaitResponse::FDriverWaitResponse(EState InState, FTimespan InNextWait)
	: NextWait(MoveTemp(InNextWait))
	, State(InState)
{ }

FWaitTimeout::FWaitTimeout(FTimespan InTimespan)
	: Timespan(MoveTemp(InTimespan))
{ }

FWaitTimeout FWaitTimeout::InMilliseconds(double Value)
{
	return FWaitTimeout(FTimespan::FromMilliseconds(Value));
}

FWaitTimeout FWaitTimeout::InSeconds(double Value)
{
	return FWaitTimeout(FTimespan::FromSeconds(Value));
}

FWaitTimeout FWaitTimeout::InMinutes(double Value)
{
	return FWaitTimeout(FTimespan::FromMinutes(Value));
}

FWaitTimeout FWaitTimeout::InHours(double Value)
{
	return FWaitTimeout(FTimespan::FromHours(Value));
}

FWaitInterval::FWaitInterval(FTimespan InTimespan)
	: Timespan(MoveTemp(InTimespan))
{ }

FWaitInterval FWaitInterval::InMilliseconds(double Value)
{
	return FWaitInterval(FTimespan::FromMilliseconds(Value));
}

FWaitInterval FWaitInterval::InSeconds(double Value)
{
	return FWaitInterval(FTimespan::FromSeconds(Value));
}

FWaitInterval FWaitInterval::InMinutes(double Value)
{
	return FWaitInterval(FTimespan::FromMinutes(Value));
}

FWaitInterval FWaitInterval::InHours(double Value)
{
	return FWaitInterval(FTimespan::FromHours(Value));
}

FDriverWaitDelegate Until::ElementExists(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout)
{
	return Until::ElementExists(ElementLocator, FWaitInterval::InSeconds(1.0), Timeout);
}

FDriverWaitDelegate Until::ElementExists(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout)
{
	return Until::Lambda([ElementLocator, Interval, Timeout](const FTimespan& TotalWaitTime){
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		if (Elements.Num() == 0)
		{
			if (TotalWaitTime > Timeout.Timespan)
			{
				return FDriverWaitResponse::Failed();
			}

			return FDriverWaitResponse::Wait(Interval.Timespan);
		}

		return FDriverWaitResponse::Passed();
	});
}

FDriverWaitDelegate Until::ElementIsVisible(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout)
{
	return Until::ElementIsVisible(ElementLocator, FWaitInterval::InSeconds(1.0), Timeout);
}

FDriverWaitDelegate Until::ElementIsVisible(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout)
{
	return Until::Lambda([ElementLocator, Interval, Timeout](const FTimespan& TotalWaitTime) {
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		if (Elements.Num() == 0)
		{
			if (TotalWaitTime > Timeout.Timespan)
			{
				return FDriverWaitResponse::Failed();
			}

			return FDriverWaitResponse::Wait(Interval.Timespan);
		}
		
		for (TSharedRef<IApplicationElement> Element : Elements)
		{
			if (!Element->IsVisible())
			{
				if (TotalWaitTime > Timeout.Timespan)
				{
					return FDriverWaitResponse::Failed();
				}

				return FDriverWaitResponse::Wait(Interval.Timespan);
			}
		}

		return FDriverWaitResponse::Passed();
	});
}

FDriverWaitDelegate Until::ElementIsHidden(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout)
{
	return Until::ElementIsHidden(ElementLocator, FWaitInterval::InSeconds(1.0), Timeout);
}

FDriverWaitDelegate Until::ElementIsHidden(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout)
{
	return Until::Lambda([ElementLocator, Interval, Timeout](const FTimespan& TotalWaitTime) {
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		if (Elements.Num() == 0)
		{
			if (TotalWaitTime > Timeout.Timespan)
			{
				return FDriverWaitResponse::Failed();
			}

			return FDriverWaitResponse::Wait(Interval.Timespan);
		}

		for (TSharedRef<IApplicationElement> Element : Elements)
		{
			if (Element->IsVisible())
			{
				if (TotalWaitTime > Timeout.Timespan)
				{
					return FDriverWaitResponse::Failed();
				}

				return FDriverWaitResponse::Wait(Interval.Timespan);
			}
		}

		return FDriverWaitResponse::Passed();
		});
}

FDriverWaitDelegate Until::ElementIsInteractable(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout)
{
	return Until::ElementIsInteractable(ElementLocator, FWaitInterval::InSeconds(1.0), Timeout);
}

FDriverWaitDelegate Until::ElementIsInteractable(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout)
{
	return Until::Lambda([ElementLocator, Interval, Timeout](const FTimespan& TotalWaitTime) {
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		if (Elements.Num() == 0)
		{
			if (TotalWaitTime > Timeout.Timespan)
			{
				return FDriverWaitResponse::Failed();
			}

			return FDriverWaitResponse::Wait(Interval.Timespan);
		}

		for (TSharedRef<IApplicationElement> Element : Elements)
		{
			if (!Element->IsVisible() || !Element->IsInteractable())
			{
				if (TotalWaitTime > Timeout.Timespan)
				{
					return FDriverWaitResponse::Failed();
				}

				return FDriverWaitResponse::Wait(Interval.Timespan);
			}
		}

		return FDriverWaitResponse::Passed();
	});
}

FDriverWaitDelegate Until::ElementIsScrolledToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout)
{
	return Until::ElementIsScrolledToBeginning(ElementLocator, FWaitInterval::InSeconds(1.0), Timeout);
}

FDriverWaitDelegate Until::ElementIsScrolledToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout)
{
	return Until::Lambda([ElementLocator, Interval, Timeout](const FTimespan& TotalWaitTime) {
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		if (Elements.Num() == 0)
		{
			if (TotalWaitTime > Timeout.Timespan)
			{
				return FDriverWaitResponse::Failed();
			}

			return FDriverWaitResponse::Wait(Interval.Timespan);
		}

		for (TSharedRef<IApplicationElement> Element : Elements)
		{
			if (!Element->IsScrolledToBeginning())
			{
				if (TotalWaitTime > Timeout.Timespan)
				{
					return FDriverWaitResponse::Failed();
				}

				return FDriverWaitResponse::Wait(Interval.Timespan);
			}
		}

		return FDriverWaitResponse::Passed();
	});
}

FDriverWaitDelegate Until::ElementIsScrolledToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout)
{
	return Until::ElementIsScrolledToEnd(ElementLocator, FWaitInterval::InSeconds(1.0), Timeout);
}

FDriverWaitDelegate Until::ElementIsScrolledToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout)
{
	return Until::Lambda([ElementLocator, Interval, Timeout](const FTimespan& TotalWaitTime) {
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		if (Elements.Num() == 0)
		{
			if (TotalWaitTime > Timeout.Timespan)
			{
				return FDriverWaitResponse::Failed();
			}

			return FDriverWaitResponse::Wait(Interval.Timespan);
		}

		for (TSharedRef<IApplicationElement> Element : Elements)
		{
			if (!Element->IsScrolledToEnd())
			{
				if (TotalWaitTime > Timeout.Timespan)
				{
					return FDriverWaitResponse::Failed();
				}

				return FDriverWaitResponse::Wait(Interval.Timespan);
			}
		}

		return FDriverWaitResponse::Passed();
	});
}

FDriverWaitDelegate Until::ElementIsFocusedByKeyboard(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator,
	FWaitTimeout Timeout)
{
	return Until::ElementIsFocusedByKeyboard(ElementLocator, FWaitInterval::InSeconds(1.0), Timeout);
}

FDriverWaitDelegate Until::ElementIsFocusedByKeyboard(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator,
	FWaitInterval Interval, FWaitTimeout Timeout)
{
	return Until::Condition([ElementLocator]()
	{
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		for (const TSharedRef<IApplicationElement>& Element : Elements)
		{
			if (Element->IsFocused())
			{
				return true;
			}
		}

		return false;
	}, Interval, Timeout);
}

FDriverWaitDelegate Until::ElementIsFocusedByUser(uint32 UserIndex,
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout)
{
	return Until::ElementIsFocusedByUser(UserIndex, ElementLocator, FWaitInterval::InSeconds(1.0), Timeout);
}

FDriverWaitDelegate Until::ElementIsFocusedByUser(uint32 UserIndex,
	const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout)
{
	return Until::Condition([ElementLocator, UserIndex]()
	{
		check(IsInGameThread());

		TArray<TSharedRef<IApplicationElement>> Elements;
		ElementLocator->Locate(Elements);

		for (const TSharedRef<IApplicationElement>& Element : Elements)
		{
			if (Element->IsFocused(UserIndex))
			{
				return true;
			}
		}

		return false;
	}, Interval, Timeout);
}

FDriverWaitDelegate Until::Condition(const TFunction<bool()>& Function, FWaitTimeout Timeout)
{
	return Until::Condition(FDriverWaitConditionDelegate::CreateLambda(Function), Timeout);
}

FDriverWaitDelegate Until::Condition(const TFunction<bool()>& Function, FWaitInterval Interval, FWaitTimeout Timeout)
{
	return Until::Condition(FDriverWaitConditionDelegate::CreateLambda(Function), Interval, Timeout);
}

FDriverWaitDelegate Until::Condition(const FDriverWaitConditionDelegate& Delegate, FWaitTimeout Timeout)
{
	return Until::Condition(Delegate, FWaitInterval::InSeconds(1.0), Timeout);
}

FDriverWaitDelegate Until::Condition(const FDriverWaitConditionDelegate& Delegate, FWaitInterval Interval, FWaitTimeout Timeout)
{
	return FDriverWaitDelegate::CreateLambda([Delegate, Interval, Timeout](const FTimespan& TotalWaitTime) {
		check(IsInGameThread());

		if (!Delegate.IsBound())
		{
			return FDriverWaitResponse::Failed();
		}
		
		if (!Delegate.Execute())
		{
			if (TotalWaitTime > Timeout.Timespan)
			{
				return FDriverWaitResponse::Failed();
			}

			return FDriverWaitResponse::Wait(Interval.Timespan);
		}

		return FDriverWaitResponse::Passed();
	});
}

FDriverWaitDelegate Until::Lambda(const TFunction<FDriverWaitResponse(const FTimespan&)>& Value)
{
	return FDriverWaitDelegate::CreateLambda(Value);
}
