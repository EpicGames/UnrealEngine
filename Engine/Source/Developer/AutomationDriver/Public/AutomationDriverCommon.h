// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/* Public Includes
*****************************************************************************/
#include "LocateBy.h"
#include "WaitUntil.h"
#include "IAutomationDriverModule.h"
#include "IAutomationDriver.h"
#include "IDriverElement.h"
#include "IDriverSequence.h"
#include "IElementLocator.h"
#include "DriverConfiguration.h"
#include "Framework/MetaData/DriverMetaData.h"
