// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointerFwd.h"

class IAutomationDriver;
class IDriverElement;
class IDriverElementCollection;
class IDriverSequence;
class IElementLocator;

typedef TSharedPtr<IAutomationDriver, ESPMode::ThreadSafe> FAutomationDriverPtr;
typedef TSharedRef<IAutomationDriver, ESPMode::ThreadSafe> FAutomationDriverRef;
typedef TSharedPtr<IDriverElement, ESPMode::ThreadSafe> FDriverElementPtr;
typedef TSharedRef<IDriverElement, ESPMode::ThreadSafe> FDriverElementRef;
typedef TSharedPtr<IDriverElementCollection, ESPMode::ThreadSafe> FDriverElementCollectionPtr;
typedef TSharedRef<IDriverElementCollection, ESPMode::ThreadSafe> FDriverElementCollectionRef;
typedef TSharedPtr<IDriverSequence, ESPMode::ThreadSafe> FDriverSequencePtr;
typedef TSharedRef<IDriverSequence, ESPMode::ThreadSafe> FDriverSequenceRef;
typedef TSharedPtr<IElementLocator, ESPMode::ThreadSafe> FElementLocatorPtr;
typedef TSharedRef<IElementLocator, ESPMode::ThreadSafe> FElementLocatorRef;

