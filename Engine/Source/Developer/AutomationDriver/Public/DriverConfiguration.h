// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Timespan.h"

/**
 * A model class that holds all the configuration options for an AutomationDriver
 */
class FDriverConfiguration
{
public:

	/**
	 * The timespan any action should conditionally wait for a scenario to occur before failing, such as, waiting for
	 * a locator to locate a specific application element.
	 */
	FTimespan ImplicitWait;

	/**
	 * The multiplier at which the automation driver should adjusts the time between individual steps by.
	 * The automation driver caps the minimum wait time to one frame.
	 */
	float ExecutionSpeedMultiplier;

	FDriverConfiguration()
		: ImplicitWait(FTimespan::FromSeconds(3))
		, ExecutionSpeedMultiplier(1.0)
	{ }
};

