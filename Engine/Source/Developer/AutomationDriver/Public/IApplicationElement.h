// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericWindow.h"
#include "Types/SlateEnums.h"

class IElementLocator;

class IApplicationElement
{
public:

	// @return a string representation of this element intended to be used solely for debugging purposes
	virtual FString ToDebugString() const = 0;

	// @return the position on this element in screen space
	virtual FVector2D GetAbsolutePosition() const = 0;

	// @return the size of the interactable area of this element on screen
	virtual FVector2D GetSize() const = 0;

	// @return the OS window containing this element
	virtual TSharedPtr<FGenericWindow> GetWindow() const = 0;

	// @return whether the element is currently visible to the user
	virtual bool IsVisible() const = 0;

	// @return whether the user can currently interact with the element
	virtual bool IsInteractable() const = 0;

	// @return whether the element is currently checked
	virtual bool IsChecked() const = 0;

	// @return the text being displayed to the user via this element
	virtual FText GetText() const = 0;

	// @return a special element locator that can be used to recall this specific element
	virtual TSharedRef<IElementLocator, ESPMode::ThreadSafe> CreateLocator() const = 0;

	// @return whether this element can be focused
	virtual bool CanFocus() const = 0;

	// @return whether focus was changed to the element for the default user
	virtual bool Focus() const = 0;

	// @return whether focus was changed to the element for the specified user
	virtual bool Focus(uint32 UserIndex) const = 0;

	// @return whether the element is currently the focus of the default user
	virtual bool IsFocused() const = 0;

	// @return whether the element is currently the focus of the specified user
	virtual bool IsFocused(uint32 UserIndex) const = 0;

	// @return whether the element is currently in the parent hierarchy of the focus of the default user
	virtual bool HasFocusedDescendants() const = 0;

	// @return whether the element is currently in the parent hierarchy of the focus of the specified user
	virtual bool HasFocusedDescendants(uint32 UserIndex) const = 0;

	// @return whether this element is currently being hovered over by the cursor
	virtual bool IsHovered() const = 0;

	/** @return whether this application element is scrollable */
	virtual bool IsScrollable() const = 0;

	/** @return whether this application element is scrollable; OutScrollOrientation will be set to the orientation of the element only if the element is scrollable */
	virtual bool IsScrollable(EOrientation& OutScrollOrientation) const = 0;

	/** @return whether the element's scroll position is at the very top; only valid on scrollable elements */
	virtual bool IsScrolledToBeginning() const = 0;

	/** @return whether the element's scroll position is at the very bottom; only valid on scrollable elements */
	virtual bool IsScrolledToEnd() const = 0;

	/** @return a parent element of this element which is scrollable; null if no scrollable parent exists */
	virtual TSharedPtr<IApplicationElement> GetScrollableParent() const = 0;

	/** @return an empty, null element */
	virtual void* GetRawElement() const = 0;
};
