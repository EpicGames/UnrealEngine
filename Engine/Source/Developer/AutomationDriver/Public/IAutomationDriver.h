// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/AsyncResult.h"
#include "Misc/Timespan.h"
#include "WaitUntil.h"
#include "GenericPlatform/GenericApplication.h"

class IElementLocator;

class IAsyncDriverSequence;
class IAsyncDriverElement;
class IAsyncDriverElementCollection;
class FDriverConfiguration;

/**
 * The AsyncAutomationDriver is the async varition of the general AutomationDriver API.
 * The AutomationDriver can create sequences and locate elements to simulate input for.
 */
class IAsyncAutomationDriver
{
public:

	/**
	 * Performs an async wait no shorter than for the specified Timespan
	 * @return true after at least the specified amount of time has elasped; may return false if the wait is forcibly interrupted
	 */
	virtual TAsyncResult<bool> Wait(FTimespan Timespan) = 0;

	/**
	 * Performs an async wait until the specified DriverWaitDelegate returns a PASSED or FAILED response
	 * @return true if the delegate ultimately returned PASSED; otherwise false
	 */
	virtual TAsyncResult<bool> Wait(const FDriverWaitDelegate& Delegate) = 0;

	/**
	 * @return a new async driver sequence which can be used to issue a serious of commands to the driver
	 */
	virtual TSharedRef<IAsyncDriverSequence, ESPMode::ThreadSafe> CreateSequence() = 0;

	/**
	 * @return the current position of the cursor
	 */
	virtual TAsyncResult<FVector2D> GetCursorPosition() const = 0;

	/**
	 * @return the current state of modifier keys for the application
	 */
	virtual TAsyncResult<FModifierKeysState> GetModifierKeys() const = 0;

	/**
	 * This is a non-blocking call and doesn't invoke the locator until some action is performed on the element. You will need to
	 * invoke the Exists() method on the element to confirm the elements existence if that is what you want to do.
	 *
	 * @return a driver element representing a single element located by the specified locator
	 */
	virtual TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe> FindElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * This is a non-blocking call and doesn't invoke the locator until some action is performed on the element. You will need to
	 * invoke the GetElements() method on the collection to confirm the existence of any elements.
	 *
	 * @return a driver element collection representing multiple elements potentially located by the specified locator
	 */
	virtual TSharedRef<IAsyncDriverElementCollection, ESPMode::ThreadSafe> FindElements(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * @return the driver's configuration details
	 */
	virtual TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe> GetConfiguration() const = 0;
};


class IDriverSequence;
class IDriverElement;
class IDriverElementCollection;
class FDriverConfiguration;

/**
 * The AutomationDriver can create sequences and locate elements to simulate input for.
 */
class IAutomationDriver
{
public:

	/**
	 * Performs an wait no shorter than for the specified Timespan
	 * @return true after at least the specified amount of time has elasped; may return false if the wait is forcibly interrupted
	 */
	virtual bool Wait(FTimespan Timespan) = 0;

	/**
	 * Performs an wait until the specified DriverWaitDelegate returns a PASSED or FAILED response
	 * @return true if the delegate ultimately returned PASSED; otherwise false
	 */
	virtual bool Wait(const FDriverWaitDelegate& Delegate) = 0;

	/**
	 * @return a new driver sequence which can be used to issue a series of commands to the driver
	 */
	virtual TSharedRef<IDriverSequence, ESPMode::ThreadSafe> CreateSequence() = 0;

	/**
	 * @return the current position of the cursor
	 */
	virtual FVector2D GetCursorPosition() const = 0;

	/**
	 * @return the current state of modifier keys for the application
	 */
	virtual FModifierKeysState GetModifierKeys() const = 0;

	/**
	 * This doesn't invoke the locator until some action is performed on the element. You will need to
	 * invoke the Exists() method on the element to confirm the elements existence if that is what you want to do.
	 *
	 * @return a driver element representing a single element located by the specified locator
	 */
	virtual TSharedRef<IDriverElement, ESPMode::ThreadSafe> FindElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * This doesn't invoke the locator until some action is performed on the element. You will need to
	 * invoke the GetElements() method on the collection to confirm the existence of any elements.
	 *
	 * @return a driver element collection representing multiple elements potentially located by the specified locator
	 */
	virtual TSharedRef<IDriverElementCollection, ESPMode::ThreadSafe> FindElements(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * @return the driver's configuration details
	 */
	virtual TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe> GetConfiguration() const = 0;
};
