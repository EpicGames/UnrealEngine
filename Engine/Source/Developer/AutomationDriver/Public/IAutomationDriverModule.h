// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IAutomationDriver;
class IAsyncAutomationDriver;
class FDriverConfiguration;

class IAutomationDriverModule
	: public IModuleInterface
{
public:

	static inline IAutomationDriverModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IAutomationDriverModule>("AutomationDriver");
	}

	/**
	 * @return a new automation driver that can be used to simulate input
	 */
	virtual TSharedRef<IAutomationDriver, ESPMode::ThreadSafe> CreateDriver() const = 0;

	/**
	 * @return a new automation driver that can be used to simulate input; using the specified configuration
	 */
	virtual TSharedRef<IAutomationDriver, ESPMode::ThreadSafe> CreateDriver(const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration) const = 0;

	/**
	 * @return a new async automation driver that can be used to simulate input
	 */
	virtual TSharedRef<IAsyncAutomationDriver, ESPMode::ThreadSafe> CreateAsyncDriver() const = 0;

	/**
	 * @return a new async automation driver that can be used to simulate input; using the specified configuration
	 */
	virtual TSharedRef<IAsyncAutomationDriver, ESPMode::ThreadSafe> CreateAsyncDriver(const TSharedRef<FDriverConfiguration, ESPMode::ThreadSafe>& Configuration) const = 0;

	/**
	 * @return whether the automation driver module is actively enabled
	 */
	virtual bool IsEnabled() const = 0;

	/**
	 * Enables the automation driver. 
	 * Enabling the automation driver module causes most traditional input messages from 
	 * the platform to stop being received, and instead only input simulated via an actual 
	 * automation driver is received.
	 */
	virtual void Enable() = 0;

	/**
	 * Disables the automation driver. 
	 * Disabling the automation driver module restores the platform specific messaging so 
	 * they are once again received by the application.
	 */
	virtual void Disable() = 0;
};
