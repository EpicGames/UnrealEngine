// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IElementLocator.h"
#include "Async/AsyncResult.h"
#include "InputCoreTypes.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

/**
 * Represents the async API for a potential application element
 */
class IAsyncDriverElement
	: public IElementLocator
{
public:

	/**
	 * Moves the cursor over the element, triggering any hover cue the element may have
	 * @return whether the cursor move was successful
	 */
	virtual TAsyncResult<bool> Hover() = 0;

	/**
	 * @return whether the element believes it is currently being hovered by the cursor
	 */
	virtual TAsyncResult<bool> IsHovered() const = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a click with the specified mouse button
	 * @return whether the click was successful
	 */
	virtual TAsyncResult<bool> Click(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a click with the left mouse button
	 * @return whether the click was successful
	 */
	virtual TAsyncResult<bool> Click() = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a double click with the specified mouse button
	 * @return whether the double click was successful
	 */
	virtual TAsyncResult<bool> DoubleClick(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a double click with the left mouse button
	 * @return whether the double click was successful
	 */
	virtual TAsyncResult<bool> DoubleClick() = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta.
	 * Most average user scrolling is limited to a range of -5 to 5.
	 * @return whether the scroll by delta was successful
	 */
	virtual TAsyncResult<bool> ScrollBy(float Delta) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll, scrolling the element 
	 * until it reaches the beginning
	 * @return whether the scroll to beginning was successful
	 */
	virtual TAsyncResult<bool> ScrollToBeginning() = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta 
	 * amount, scrolling the element until it reaches the beginning
	 * @return whether the scroll to beginning was successful
	 */
	virtual TAsyncResult<bool> ScrollToBeginning(float Amount) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * is locates the desired element or the beginning is reached
	 * @return whether the scroll to beginning until was successful
	 */
	virtual TAsyncResult<bool> ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll, scrolling the element until
	 * it reaches the beginning
	 * @return whether the scroll to end was successful
	 */
	virtual TAsyncResult<bool> ScrollToEnd() = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta
	 * amount, scrolling the element until it reaches the end
	 * @return whether the scroll to end was successful
	 */
	virtual TAsyncResult<bool> ScrollToEnd(float Amount) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * is locates the desired element or the end is reached
	 * @return whether the scroll to end until was successful
	 */
	virtual TAsyncResult<bool> ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 * @return whether the type was successful
	 */
	virtual TAsyncResult<bool> Type(const TCHAR* Text) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 * @return whether the type was successful
	 */
	virtual TAsyncResult<bool> Type(FString Text) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for the specified character.
	 * @return whether the type was successful
	 */
	virtual TAsyncResult<bool> Type(FKey Key) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for the specified key.
	 * Not all control and modifier keys are supported with this method of typing.
	 * @return whether the type was successful
	 */
	virtual TAsyncResult<bool> Type(TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each key specified.
	 * @return whether the type was successful
	 */
	virtual TAsyncResult<bool> Type(const TArray<FKey>& Keys) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 * @return whether the type chord was successful
	 */
	virtual TAsyncResult<bool> TypeChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 * @return whether the type chord was successful
	 */
	virtual TAsyncResult<bool> TypeChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 * @return whether the type chord was successful
	 */
	virtual TAsyncResult<bool> TypeChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 * @return whether the type chord was successful
	 */
	virtual TAsyncResult<bool> TypeChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for pressing the 
	 * specified character. The character will not be released, nor will their be repeat character events issued for
	 * holding the character.
	 * @return whether the press was successful
	 */
	virtual TAsyncResult<bool> Press(TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for pressing the 
	 * specified key. The key will not be released, nor will their be repeat key events issued for
	 * holding the key.
	 * @return whether the press was successful
	 */
	virtual TAsyncResult<bool> Press(FKey Key) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke mouse input for pressing the 
	 * specified button. The button will not be released.
	 * @return whether the press was successful
	 */
	virtual TAsyncResult<bool> Press(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 * @return whether the press chord was successful
	 */
	virtual TAsyncResult<bool> PressChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 * @return whether the press chord was successful
	 */
	virtual TAsyncResult<bool> PressChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 * @return whether the press chord was successful
	 */
	virtual TAsyncResult<bool> PressChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 * @return whether the press chord was successful
	 */
	virtual TAsyncResult<bool> PressChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for releasing the 
	 * specified character. The character will not be pressed if it is not currently being pressed.
	 * @return whether the release was successful
	 */
	virtual TAsyncResult<bool> Release(TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for releasing the 
	 * specified key. The key will not be pressed if it is not currently being pressed.
	 * @return whether the release was successful
	 */
	virtual TAsyncResult<bool> Release(FKey Key) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke mouse input for releasing the 
	 * specified button. The button will not be pressed if it is not currently pressed.
	 * @return whether the release was successful
	 */
	virtual TAsyncResult<bool> Release(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 * @return whether the release chord was successful
	 */
	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 * @return whether the release chord was successful
	 */
	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 * @return whether the release chord was successful
	 */
	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 * @return whether the release chord was successful
	 */
	virtual TAsyncResult<bool> ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Moves focus to the element for the default user
	 * @return whether the focus change was successful
	 */
	virtual TAsyncResult<bool> Focus() = 0;

	/**
	 * Moves focus to the element for the specified user
	 * @return whether the focus change was successful
	 */
	virtual TAsyncResult<bool> Focus(uint32 UserIndex) = 0;

	/**
	 * @return whether the element can be focused
	 */
	virtual TAsyncResult<bool> CanFocus() const = 0;

	/**
	 * @return whether the element is currently the focus for the default user
	 */
	virtual TAsyncResult<bool> IsFocused() const = 0;

	/**
	 * @return whether the element is currently the focus for the specified user
	 */
	virtual TAsyncResult<bool> IsFocused(uint32 UserIndex) const = 0;

	/**
	 * @return whether the element is currently in the parent hierarchy of the focus of the default user
	 */
	virtual TAsyncResult<bool> HasFocusedDescendants() const = 0;

	/**
	 * @return whether the element is currently in the parent hierarchy of the focus of the specified user
	 */
	virtual TAsyncResult<bool> HasFocusedDescendants(uint32 UserIndex) const = 0;

	/**
	 * @return whether the element is currently exists
	 */
	virtual TAsyncResult<bool> Exists() const = 0;

	/**
	 * @return whether the element is currently visible; the element must be on screen
	 */
	virtual TAsyncResult<bool> IsVisible() const = 0;

	/**
	 * @return whether the element is currently checked; will return false if the element is uncheckable
	 */
	virtual TAsyncResult<bool> IsChecked() const = 0;

	/**
	 * @return whether the element is currently interactable; the element must be on screen
	 */
	virtual TAsyncResult<bool> IsInteractable() const = 0;

	/**
	 * @return whether the element is scrollable; an element is considered scrollable if it has any scrollable descendants
	 */
	virtual TAsyncResult<bool> IsScrollable() const = 0;

	/**
	 * @return is true if the element is scrollable and at the beginning position
	 */
	virtual TAsyncResult<bool> IsScrolledToBeginning() const = 0;

	/**
	 * @return is true if the element is scrollable and at the end position
	 */
	virtual TAsyncResult<bool> IsScrolledToEnd() const = 0;

	/**
	 * @return the screen space absolute position of the element's top left corner 
	 */
	virtual TAsyncResult<FVector2D> GetAbsolutePosition() const = 0;

	/**
	 * @return the screen size of the element
	 */
	virtual TAsyncResult<FVector2D> GetSize() const = 0;

	/**
	 * @return the text displayed by the element; if the element does not display text itself, then if there is only 
	 * a single descendant of the element that displays text, then that descendants text will be returned
	 */
	virtual TAsyncResult<FText> GetText() const = 0;
};


/**
 * Represents a potential collection of elements 
 */
class IAsyncDriverElementCollection
{
public:

	/**
	 * @return the elements discovered by the locator at this current moment; calling this multiple times may return different elements
	 */
	virtual TAsyncResult<TArray<TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe>>> GetElements() = 0;
};

class FEmptyAsyncDriverElementFactory
{
public:
	static TSharedRef<IAsyncDriverElement, ESPMode::ThreadSafe> Create();
};

/**
 * Represents the API for a potential application element
 */
class IDriverElement
	: public IElementLocator
{
public:

	/**
	 * Moves the cursor over the element, triggering any hover cue the element may have
	 * @return whether the cursor move was successful
	 */
	virtual bool Hover() = 0;

	/**
	 * @return whether the element believes it is currently being hovered by the cursor
	 */
	virtual bool IsHovered() const = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a click with the specified mouse button
	 * @return whether the click was successful
	 */
	virtual bool Click(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a click with the left mouse button
	 * @return whether the click was successful
	 */
	virtual bool Click() = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a double click with the specified mouse button
	 * @return whether the double click was successful
	 */
	virtual bool DoubleClick(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a double click with the left mouse button
	 * @return whether the double click was successful
	 */
	virtual bool DoubleClick() = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta.
	 * Most average user scrolling is limited to a range of -5 to 5.
	 * @return whether the scroll by delta was successful
	 */
	virtual bool ScrollBy(float Delta) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll, scrolling the element 
	 * until it reaches the beginning
	 * @return whether the scroll to beginning was successful
	 */
	virtual bool ScrollToBeginning() = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta 
	 * amount, scrolling the element until it reaches the beginning
	 * @return whether the scroll to beginning was successful
	 */
	virtual bool ScrollToBeginning(float Amount) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * is locates the desired element or the beginning is reached
	 * @return whether the scroll to beginning until was successful
	 */
	virtual bool ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll, scrolling the element until
	 * it reaches the beginning
	 * @return whether the scroll to end was successful
	 */
	virtual bool ScrollToEnd() = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta
	 * amount, scrolling the element until it reaches the end
	 * @return whether the scroll to end was successful
	 */
	virtual bool ScrollToEnd(float Amount) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * is locates the desired element or the end is reached
	 * @return whether the scroll to end until was successful
	 */
	virtual bool ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& DesiredElementLocator) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 * @return whether the type was successful
	 */
	virtual bool Type(const TCHAR* Text) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 * @return whether the type was successful
	 */
	virtual bool Type(FString Text) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for the specified character.
	 * @return whether the type was successful
	 */
	virtual bool Type(FKey Key) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for the specified key.
	 * Not all control and modifier keys are supported with this method of typing.
	 * @return whether the type was successful
	 */
	virtual bool Type(TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each key specified.
	 * @return whether the type was successful
	 */
	virtual bool Type(const TArray<FKey>& Keys) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 * @return whether the type chord was successful
	 */
	virtual bool TypeChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 * @return whether the type chord was successful
	 */
	virtual bool TypeChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 * @return whether the type chord was successful
	 */
	virtual bool TypeChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 * @return whether the type chord was successful
	 */
	virtual bool TypeChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for pressing the 
	 * specified character. The character will not be released, nor will their be repeat character events issued for
	 * holding the character.
	 * @return whether the press was successful
	 */
	virtual bool Press(TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for pressing the 
	 * specified key. The key will not be released, nor will their be repeat key events issued for
	 * holding the key.
	 * @return whether the press was successful
	 */
	virtual bool Press(FKey Key) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke mouse input for pressing the 
	 * specified button. The button will not be released.
	 * @return whether the press was successful
	 */
	virtual bool Press(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 * @return whether the press chord was successful
	 */
	virtual bool PressChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 * @return whether the press chord was successful
	 */
	virtual bool PressChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 * @return whether the press chord was successful
	 */
	virtual bool PressChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 * @return whether the press chord was successful
	 */
	virtual bool PressChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for releasing the 
	 * specified character. The character will not be pressed if it is not currently being pressed.
	 * @return whether the release was successful
	 */
	virtual bool Release(TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for releasing the 
	 * specified key. The key will not be pressed if it is not currently being pressed.
	 * @return whether the release was successful
	 */
	virtual bool Release(FKey Key) = 0;

	/**
	 * Moves the cursor over the element, then triggers the series of events to invoke mouse input for releasing the 
	 * specified button. The button will not be pressed if it is not currently pressed.
	 * @return whether the release was successful
	 */
	virtual bool Release(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if if not currently being pressed.
	 * @return whether the release chord was successful
	 */
	virtual bool ReleaseChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if if not currently being pressed.
	 * @return whether the release chord was successful
	 */
	virtual bool ReleaseChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if if not currently being pressed.
	 * @return whether the release chord was successful
	 */
	virtual bool ReleaseChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Moves focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if if not currently being pressed.
	 * @return whether the release chord was successful
	 */
	virtual bool ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Moves focus to the element for the default user
	 * @return whether the focus change was successful
	 */
	virtual bool Focus() = 0;
	
	/**
	 * Moves focus to the element for the specified user
	 * @return whether the focus change was successful
	 */
	virtual bool Focus(uint32 UserIndex) = 0;

	/**
	 * @return whether the element can be focused
	 */
	virtual bool CanFocus() const = 0;

	/**
	 * @return whether the element is currently the focus for the default user
	 */
	virtual bool IsFocused() const = 0;

	/**
	 * @return whether the element is currently the focus for the specified user
	 */
	virtual bool IsFocused(uint32 UserIndex) const = 0;

	/**
	 * @return whether the element is currently in the parent hierarchy of the focus of the default user
	 */
	virtual bool HasFocusedDescendants() const = 0;

	/**
	 * @return whether the element is currently in the parent hierarchy of the focus of the specified user
	 */
	virtual bool HasFocusedDescendants(uint32 UserIndex) const = 0;

	/**
	 * @return whether the element is currently exists
	 */
	virtual bool Exists() const = 0;

	/**
	 * @return whether the element is currently visible; the element must be on screen
	 */
	virtual bool IsVisible() const = 0;

	/**
	 * @return whether the element is currently checked; will return false if the element is uncheckable
	 */
	virtual bool IsChecked() const = 0;

	/**
	 * @return whether the element is currently interactable; the element must be on screen
	 */
	virtual bool IsInteractable() const = 0;

	/**
	 * @return whether the element is scrollable; an element is considered scrollable if it has any scrollable descendants
	 */
	virtual bool IsScrollable() const = 0;

	/**
	 * @return is true if the element is scrollable and at the beginning position
	 */
	virtual bool IsScrolledToBeginning() const = 0;

	/**
	 * @return is true if the element is scrollable and at the end position
	 */
	virtual bool IsScrolledToEnd() const = 0;

	/**
	 * @return the screen space absolute position of the element's top left corner 
	 */
	virtual FVector2D GetAbsolutePosition() const = 0;

	/**
	 * @return the screen size of the element
	 */
	virtual FVector2D GetSize() const = 0;

	/**
	 * @return the text displayed by the element; if the element does not display text itself, then if there is only 
	 * a single descendant of the element that displays text, then that descendants text will be returned
	 */
	virtual FText GetText() const = 0;
};

/**
 * Represents a potential collection of elements 
 */
class IDriverElementCollection
{
public:

	/**
	 * @return the elements discovered by the locator at this current moment; calling this multiple times may return different elements
	 */
	virtual TArray<TSharedRef<IDriverElement, ESPMode::ThreadSafe>> GetElements() = 0;
};

class FEmptyDriverElementFactory
{
public:
	static TSharedRef<IDriverElement, ESPMode::ThreadSafe> Create();
};
