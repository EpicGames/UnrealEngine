// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WaitUntil.h"
#include "Misc/Timespan.h"
#include "InputCoreTypes.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

class IElementLocator;
template<typename ResultType> class TAsyncResult;

class IAsyncActionSequence
{
public:

	/**
	 * Queues a wait no shorter than for the specified Timespan
	 */
	virtual IAsyncActionSequence& Wait(FTimespan Timespan) = 0;

	/**
	 * Queues a wait until the specified DriverWaitDelegate returns a PASSED or FAILED response
	 */
	virtual IAsyncActionSequence& Wait(const FDriverWaitDelegate& Delegate) = 0;

	/**
	 * Queues a move of the cursor over the center of the element offset by the specified amounts, triggering any hover cue the element may have
	 */
	virtual IAsyncActionSequence& MoveToElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, int32 XOffset, int32 YOffset) = 0;

	/**
	 * Queues a move of the cursor over the center of the element, triggering any hover cue the element may have
	 */
	virtual IAsyncActionSequence& MoveToElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor from its current position by the specified offsets
	 */
	virtual IAsyncActionSequence& MoveByOffset(int32 XOffset, int32 YOffset) = 0;

	/**
	 * Queues a move of the cursor from its current position to a new position which is specified
	 */
	virtual IAsyncActionSequence& MoveToPosition(const FVector2D& Position) = 0;
	
	/**
	 * Queues the series of events to invoke a mouse wheel scroll by the specified delta at the current mouse position
	 * Most average user scrolling is limited to a range of -5 to 5.
	 */
	virtual IAsyncActionSequence& ScrollBy(float Delta) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta
	 */
	virtual IAsyncActionSequence& ScrollBy(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Delta) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll, scrolling the element 
	 * until it reaches the beginning
	 */
	virtual IAsyncActionSequence& ScrollToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta
	 * amount, scrolling the element until it reaches the beginning
	 */
	virtual IAsyncActionSequence& ScrollToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Amount) = 0;

	/**
	 * Queues the series of events to invoke a mouse wheel scroll until the specified locator is locates the desired element or the beginning is 
	 * reached at the current cursor position
	 */
	virtual IAsyncActionSequence& ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;
	
	/**
	 * Queues a move of the cursor over the scrollable element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * locates the desired element or the beginning is reached
	 */
	virtual IAsyncActionSequence& ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ScrollableElementLocator, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll, scrolling the element 
	 * until it reaches the end
	 */
	virtual IAsyncActionSequence& ScrollToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;
	
	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta
	 * amount, scrolling the element until it reaches the end
	 */
	virtual IAsyncActionSequence& ScrollToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Delta) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * locates the desired element or the end is reached
	 */
	virtual IAsyncActionSequence& ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;
	
	/**
	 * Queues a move of the cursor over the scrollable element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * locates the desired element or the end is reached
	 */
	virtual IAsyncActionSequence& ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ScrollableElementLocator, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a click with the left mouse button
	 */
	virtual IAsyncActionSequence& Click(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a click with the specified mouse button
	 */
	virtual IAsyncActionSequence& Click(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke a click with the specified mouse button at the current cursor location
	 */
	virtual IAsyncActionSequence& Click(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke a click with the left mouse button at the current cursor location
	 */
	virtual IAsyncActionSequence& Click() = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a double click with the left mouse button
	 */
	virtual IAsyncActionSequence& DoubleClick(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a double click with the specified mouse button
	 */
	virtual IAsyncActionSequence& DoubleClick(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke a double click with the specified mouse button at the current cursor location
	 */
	virtual IAsyncActionSequence& DoubleClick(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke a double click with the left mouse button at the current cursor location
	 */
	virtual IAsyncActionSequence& DoubleClick() = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IAsyncActionSequence& Type(const TCHAR* Text) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each character of the string specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IAsyncActionSequence& Type(FString Text) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for the key specified.
	 */
	virtual IAsyncActionSequence& Type(FKey Key) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for the character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IAsyncActionSequence& Type(TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each key specified.
	 */
	virtual IAsyncActionSequence& Type(const TArray<FKey>& Keys) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const TCHAR* Text) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each character of the string specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FString Text) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for the key specified.
	 */
	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for the character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each key specified.
	 */
	virtual IAsyncActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const TArray<FKey>& Keys) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IAsyncActionSequence& TypeChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IAsyncActionSequence& TypeChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IAsyncActionSequence& TypeChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IAsyncActionSequence& TypeChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IAsyncActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IAsyncActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IAsyncActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IAsyncActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for pressing the specified character. The character will not be released, 
	 * nor will their be repeat character events issued for holding the character.
	 */
	virtual IAsyncActionSequence& Press(TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for pressing the specified key. The key will not be released, 
	 * nor will their be repeat key events issued for holding the character.
	 */
	virtual IAsyncActionSequence& Press(FKey Key) = 0;

	/**
	 * Queues the series of events to invoke mouse input for pressing the specified button. The button will not be released, 
	 * nor will their be repeat button events issued for holding the button.
	 */
	virtual IAsyncActionSequence& Press(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues a change of focus to the specified element, then a series of events to invoke keyboard input for pressing 
	 * the specified character. The character will not be released, nor will their be repeat character events issued for holding the character.
	 */
	virtual IAsyncActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the specified element, then a series of events to invoke keyboard input for pressing the specified key. 
	 * The key will not be released, nor will their be repeat key events issued for holding the character.
	 */
	virtual IAsyncActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key) = 0;

	/**
	 * Queues a move of the cursor over the element, then a series of events to invoke mouse input for pressing the specified button. The 
	 * button will not be released, nor will their be repeat button events issued for holding the button.
	 */
	virtual IAsyncActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IAsyncActionSequence& PressChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IAsyncActionSequence& PressChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IAsyncActionSequence& PressChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IAsyncActionSequence& PressChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IAsyncActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IAsyncActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IAsyncActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IAsyncActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for releasing the 
	 * specified character. The character will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& Release(TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for releasing the 
	 * specified key. The key will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& Release(FKey Key) = 0;

	/**
	 * Queues the series of events to invoke mouse input for releasing the 
	 * specified button. The button will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& Release(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for releasing the 
	 * specified character. The character will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for releasing the 
	 * specified character. The character will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke mouse input for releasing the 
	 * specified button. The button will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& ReleaseChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& ReleaseChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& ReleaseChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IAsyncActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element for the default user
	 */
	virtual IAsyncActionSequence& Focus(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a change of focus to the element for the specified user
	 */
	virtual IAsyncActionSequence& Focus(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, uint32 UserFocus) = 0;
	
	/**
	 * Checks whether the sequence is currently executing actions
	 */
	virtual bool IsExecuting() = 0;
};

/**
 * Represents a sequence of actions that can be performed as a single unit by the automation driver associted with this sequence
 */
class IAsyncDriverSequence
{
public:

	/**
	 * @return the sequence of actions this driver sequence represents; any actions called on this are queued for execution on the next Perform call
	 */
	virtual IAsyncActionSequence& Actions() = 0;

	/**
	 * Executes all the actions that have been queued for this sequence. Additional actions cannot be queued while the sequence is being performed
	 */
	virtual TAsyncResult<bool> Perform() = 0;
};

class IActionSequence
{
public:

	/**
	 * Queues a wait no shorter than for the specified Timespan
	 */
	virtual IActionSequence& Wait(FTimespan Timespan) = 0;

	/**
	 * Queues a wait until the specified DriverWaitDelegate returns a PASSED or FAILED response
	 */
	virtual IActionSequence& Wait(const FDriverWaitDelegate& Delegate) = 0;

	/**
	 * Queues a move of the cursor over the center of the element offset by the specified amounts, triggering any hover cue the element may have
	 */
	virtual IActionSequence& MoveToElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, int32 XOffset, int32 YOffset) = 0;

	/**
	 * Queues a move of the cursor over the center of the element, triggering any hover cue the element may have
	 */
	virtual IActionSequence& MoveToElement(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;
	
	/**
	 * Queues a move of the cursor from its current position by the specified offsets
	 */
	virtual IActionSequence& MoveByOffset(int32 XOffset, int32 YOffset) = 0;

	/**
	 * Queues a move of the cursor from its current position to a new position which is specified
	 */
	virtual IActionSequence& MoveToPosition(const FVector2D& Position) = 0;
	
	/**
	 * Queues the series of events to invoke a mouse wheel scroll by the specified delta at the current mouse position
	 * Most average user scrolling is limited to a range of -5 to 5.
	 */
	virtual IActionSequence& ScrollBy(float Delta) = 0;
	
	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta
	 */
	virtual IActionSequence& ScrollBy(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Delta) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll, scrolling the element 
	 * until it reaches the beginning
	 */
	virtual IActionSequence& ScrollToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta
	 * amount, scrolling the element until it reaches the beginning
	 */
	virtual IActionSequence& ScrollToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Amount) = 0;
	
	/**
	 * Queues the series of events to invoke a mouse wheel scroll until the specified locator is locates the desired element or the beginning is 
	 * reached at the current cursor position
	 */
	virtual IActionSequence& ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the scrollable element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * locates the desired element or the beginning is reached
	 */
	virtual IActionSequence& ScrollToBeginningUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ScrollableElementLocator, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll, scrolling the element 
	 * until it reaches the end
	 */
	virtual IActionSequence& ScrollToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll by the specified delta
	 * amount, scrolling the element until it reaches the end
	 */
	virtual IActionSequence& ScrollToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, float Amount) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * locates the desired element or the end is reached
	 */
	virtual IActionSequence& ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the scrollable element, then triggers the series of events to invoke a mouse wheel scroll until the specified locator
	 * locates the desired element or the end is reached
	 */
	virtual IActionSequence& ScrollToEndUntil(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ScrollableElementLocator, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a click with the left mouse button
	 */
	virtual IActionSequence& Click(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a click with the specified mouse button
	 */
	virtual IActionSequence& Click(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke a click with the specified mouse button at the current cursor location
	 */
	virtual IActionSequence& Click(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke a click with the left mouse button at the current cursor location
	 */
	virtual IActionSequence& Click() = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a double click with the left mouse button
	 */
	virtual IActionSequence& DoubleClick(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke a double click with the specified mouse button
	 */
	virtual IActionSequence& DoubleClick(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke a double click with the specified mouse button at the current cursor location
	 */
	virtual IActionSequence& DoubleClick(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke a double click with the left mouse button at the current cursor location
	 */
	virtual IActionSequence& DoubleClick() = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IActionSequence& Type(const TCHAR* Text) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each character of the string specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IActionSequence& Type(FString Text) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for the key specified.
	 */
	virtual IActionSequence& Type(FKey Key) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for the character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IActionSequence& Type(TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each key specified.
	 */
	virtual IActionSequence& Type(const TArray<FKey>& Keys) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const TCHAR* Text) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each character of the string specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FString Text) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for the key specified.
	 */
	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for the character specified.
	 * Not all control and modifier keys are supported with this method of typing.
	 */
	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each key specified.
	 */
	virtual IActionSequence& Type(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, const TArray<FKey>& Keys) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IActionSequence& TypeChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IActionSequence& TypeChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IActionSequence& TypeChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IActionSequence& TypeChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues the a change of focus to the specified element, then a series of events to invoke keyboard input for each argument 
	 * in order from 1 to N and then releasing those keys in order from N to 1
	 * Chord typing is intended to ease the to execution keyboard shortcuts, such as, CTRL + SHIFT + S
	 */
	virtual IActionSequence& TypeChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for pressing the specified character. The character will not be released, 
	 * nor will their be repeat character events issued for holding the character.
	 */
	virtual IActionSequence& Press(TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for pressing the specified key. The key will not be released, 
	 * nor will their be repeat key events issued for holding the character.
	 */
	virtual IActionSequence& Press(FKey Key) = 0;

	/**
	 * Queues the series of events to invoke mouse input for pressing the specified button. The button will not be released, 
	 * nor will their be repeat button events issued for holding the button.
	 */
	virtual IActionSequence& Press(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues a change of focus to the specified element, then a series of events to invoke keyboard input for pressing 
	 * the specified character. The character will not be released, nor will their be repeat character events issued for holding the character.
	 */
	virtual IActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the specified element, then a series of events to invoke keyboard input for pressing the specified key. 
	 * The key will not be released, nor will their be repeat key events issued for holding the character.
	 */
	virtual IActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key) = 0;

	/**
	 * Queues a move of the cursor over the element, then a series of events to invoke mouse input for pressing the specified button. The 
	 * button will not be released, nor will their be repeat button events issued for holding the button.
	 */
	virtual IActionSequence& Press(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IActionSequence& PressChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IActionSequence& PressChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IActionSequence& PressChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IActionSequence& PressChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from 1 to N. They will not be released.
	 */
	virtual IActionSequence& PressChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for releasing the 
	 * specified character. The character will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& Release(TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for releasing the 
	 * specified key. The key will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& Release(FKey Key) = 0;

	/**
	 * Queues the series of events to invoke mouse input for releasing the 
	 * specified button. The button will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& Release(EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for releasing the 
	 * specified character. The character will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for releasing the 
	 * specified character. The character will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key) = 0;

	/**
	 * Queues a move of the cursor over the element, then triggers the series of events to invoke mouse input for releasing the 
	 * specified button. The button will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& Release(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, EMouseButtons::Type MouseButton) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& ReleaseChord(FKey Key1, FKey Key2) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& ReleaseChord(FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& ReleaseChord(FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& ReleaseChord(FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, FKey Key3) = 0;

	/**
	 * Queues a change of focus to the element, then triggers the series of events to invoke keyboard input for each argument
	 * in order from N to 1. They will not be pressed if it is not currently being pressed.
	 */
	virtual IActionSequence& ReleaseChord(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FKey Key1, FKey Key2, TCHAR Character) = 0;

	/**
	 * Queues a change of focus to the element for the default user
	 */
	virtual IActionSequence& Focus(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator) = 0;

	/**
	 * Queues a change of focus to the element for the specified user
	 */
	virtual IActionSequence& Focus(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, uint32 UserFocus) = 0;
};

/**
 * Represents a sequence of actions that can be performed as a single unit by the automation driver associted with this sequence
 */
class IDriverSequence
{
public:

	/**
	 * @return the sequence of actions this driver sequence represents; any actions called on this are queued for execution on the next Perform call
	 */
	virtual IActionSequence& Actions() = 0;

	/**
	 * Executes all the actions that have been queued for this sequence.
	 */
	virtual bool Perform() = 0;
};