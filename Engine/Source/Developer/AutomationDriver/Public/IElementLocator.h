// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

//@todo this type shouldn't be public at all
class IApplicationElement; 

/**
 * Represents an object whose sole purpose is to discover one or more application elements
 */
class IElementLocator
{
public:

	/**
	 * @return a string only to be used for debugging and logging purposes that represents this locator
	 */
	virtual FString ToDebugString() const = 0;

	/**
	 * @return Attempts to locate a specific element or set of elements in the application
	 */
	virtual void Locate(TArray<TSharedRef<IApplicationElement>>& OutElements) const = 0;
};
