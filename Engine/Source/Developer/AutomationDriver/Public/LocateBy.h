// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "AutomationDriverTypeDefs.h"

#define UE_API AUTOMATIONDRIVER_API

class IElementLocator;

DECLARE_DELEGATE_OneParam(FLocateSlateWidgetElementDelegate, TArray<TSharedRef<SWidget>>& /*OutWidgets*/);
DECLARE_DELEGATE_OneParam(FLocateSlateWidgetPathElementDelegate, TArray<FWidgetPath>& /*OutWidgetPaths*/);

/**
 * Represents a collection of fluent helper functions designed to make accessing and creating element locators easy
 */
class By
{
public:

	/**
	 * Creates a new element locator that exposes the collection of SWidgets returned from the FLocateSlateWidgetElementDelegate
	 * as discovered elements
	 * @param Value - The delegate to use
	 * @param DebugName - The string to display for errors when no elements can be located
	 * @return a locator which uses the specified delegate to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Delegate(const FLocateSlateWidgetElementDelegate& Value, FStringView DebugName = { });

	/**
	 * Creates a new element locator that exposes the collection of FWidgetPaths returned from the FLocateSlateWidgetElementDelegate
	 * as discovered elements
	 * @param Value - The delegate to use
	 * @param DebugName - The string to display for errors when no elements can be located
	 * @return a locator which uses the specified delegate to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Delegate(const FLocateSlateWidgetPathElementDelegate& Value, FStringView DebugName = { });

	/**
	 * Creates a new element locator that exposes the collection of SWidgets returned from the lambda
	 * as discovered elements
	 * @param Value - The lambda to use
	 * @param DebugName - The string to display for errors when no elements can be located
	 * @return a locator which uses the specified lambda to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> WidgetLambda(const TFunction<void(TArray<TSharedRef<SWidget>>&)>& Value, FStringView DebugName = { });

	/**
	 * Creates a new element locator that exposes the collection of FWidgetPaths returned from the lambda
	 * as discovered elements
	 * @param Value - The lambda to use
	 * @param DebugName - The string to display for errors when no elements can be located
	 * @return a locator which uses the specified lambda to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> WidgetPathLambda(const TFunction<void(TArray<FWidgetPath>&)>& Value, FStringView DebugName = { });

	/**
	 * Creates a new element locator that limits its discovered elements to ones with the specified Id
	 * @param Value - The MetaData ID of the element to find
	 * @return a locator which uses the specified Id to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Id(const FString& Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones with the specified Id,
	 * starting from the Root element given to the function
	 * @param Root - The reference to the element where the search will be started from
	 * @param Value - The MetaData ID of the element to find
	 * @return a locator which uses the specified Id to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Id(const FDriverElementRef& Root, const FString& Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones with the specified Id
	 * @param Value - The MetaData ID of the element to find
	 * @return a locator which uses the specified Id to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Id(const FName& Value);
	
	/**
	 * Creates a new element locator that limits its discovered elements to ones with the specified Id,
	 * starting from the Root element given to the function
	 * @param Root - The reference to the element where the search will be started from
	 * @param Value - The MetaData ID of the element to find
	 * @return a locator which uses the specified Id to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Id(const FDriverElementRef& Root, const FName& Value);


	/**
	 * Creates a new element locator that limits its discovered elements to ones with the specified Id
	 * @param Value - The MetaData ID of the element to find
	 * @return a locator which uses the specified Id to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Id(const TCHAR* Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones with the specified Id,
	 * starting from the Root element given to the function
	 * @param Root - The reference to the element where the search will be started from
	 * @param Value - The MetaData ID of the element to find
	 * @return a locator which uses the specified Id to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Id(const FDriverElementRef& Root, const TCHAR* Value);


	/**
	 * Creates a new element locator that limits its discovered elements to ones with the specified Id
	 * @param Value - The MetaData ID of the element to find
	 * @return a locator which uses the specified Id to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Id(const char* Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones with the specified Id,
	 * starting from the Root element given to the function
	 * @param Root - The reference to the element where the search will be started from
	 * @param Value - The MetaData ID of the element to find
	 * @return a locator which uses the specified Id to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Id(const FDriverElementRef& Root, const char* Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones matching the specified "path"
	 *
	 * Path Example: "#Suite//Piano/Key//<STextBlock>"
	 *
	 * Path Syntax:
	 *
	 * #Suite = # represents that the following text is an explicit Id, in the case of a SWidget it needs to be tagged with the driver Id metadata 
	 * Piano = plain text represents general tags, in the case of a SWidget it needs to have a Tag or TagMetadata with the appropriate plain text value
	 * <STextBlock> = <> represents types, in the case of a SWidget it should be the explicit type used in the SNew construction
	 *
	 * Hierarchy is represented by forward slashes
	 *
	 * / = a single forward slash represents that the next value must match a direct child of the element matched before it
	 * // = a double forward slash represents that the next value must match any descendant of the element matched before it
	 *
	 * Reference the AutomationDriver.spec.cpp expectations for additional examples of the syntax in use 
	 * 
	 * @param Value - The path to use
	 * @return a locator which uses the specified path to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Path(const FString& Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones matching the specified "path",
	 * starting from the Root element given to the function
	 *
	 * Path Example: "#Suite//Piano/Key//<STextBlock>"
	 *
	 * Path Syntax:
	 *
	 * #Suite = # represents that the following text is an explicit Id, in the case of a SWidget it needs to be tagged with the driver Id metadata
	 * Piano = plain text represents general tags, in the case of a SWidget it needs to have a Tag or TagMetadata with the appropriate plain text value
	 * <STextBlock> = <> represents types, in the case of a SWidget it should be the explicit type used in the SNew construction
	 *
	 * Hierarchy is represented by forward slashes
	 *
	 * / = a single forward slash represents that the next value must match a direct child of the element matched before it
	 * // = a double forward slash represents that the next value must match any descendant of the element matched before it
	 *
	 * Reference the AutomationDriver.spec.cpp expectations for additional examples of the syntax in use
	 *
	 * @param Root - The reference to the element where the search will be started from
	 * @param Value - The path to use
	 * @return a locator which uses the specified path to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Path(const FDriverElementRef& Root, const FString& Value);

	
	/**
	 * Creates a new element locator that limits its discovered elements to ones matching the specified "path"
	 *
	 * Path Example: "#Suite//Piano/Key//<STextBlock>"
	 *
	 * Path Syntax:
	 *
	 * #Suite = # represents that the following text is an explicit Id, in the case of a SWidget it needs to be tagged with the driver Id metadata 
	 * Piano = plain text represents general tags, in the case of a SWidget it needs to have a Tag or TagMetadata with the appropriate plain text value
	 * <STextBlock> = <> represents types, in the case of a SWidget it should be the explicit type used in the SNew construction
	 *
	 * Hierarchy is represented by forward slashes
	 *
	 * / = a single forward slash represents that the next value must match a direct child of the element matched before it
	 * // = a double forward slash represents that the next value must match any descendant of the element matched before it
	 *
	 * Reference the AutomationDriver.spec.cpp expectations for additional examples of the syntax in use 
	 *
	 * @param Value - The path to use
	 * @return a locator which uses the specified path to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Path(const FName& Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones matching the specified "path",
	 * starting from the Root element given to the function
	 *
	 * Path Example: "#Suite//Piano/Key//<STextBlock>"
	 *
	 * Path Syntax:
	 *
	 * #Suite = # represents that the following text is an explicit Id, in the case of a SWidget it needs to be tagged with the driver Id metadata
	 * Piano = plain text represents general tags, in the case of a SWidget it needs to have a Tag or TagMetadata with the appropriate plain text value
	 * <STextBlock> = <> represents types, in the case of a SWidget it should be the explicit type used in the SNew construction
	 *
	 * Hierarchy is represented by forward slashes
	 *
	 * / = a single forward slash represents that the next value must match a direct child of the element matched before it
	 * // = a double forward slash represents that the next value must match any descendant of the element matched before it
	 *
	 * Reference the AutomationDriver.spec.cpp expectations for additional examples of the syntax in use
	 *
	 * @param Root - The reference to the element where the search will be started from
	 * @param Value - The path to use
	 * @return a locator which uses the specified path to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Path(const FDriverElementRef& Root, const FName& Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones matching the specified "path"
	 *
	 * Path Example: "#Suite//Piano/Key//<STextBlock>"
	 *
	 * Path Syntax:
	 *
	 * #Suite = # represents that the following text is an explicit Id, in the case of a SWidget it needs to be tagged with the driver Id metadata 
	 * Piano = plain text represents general tags, in the case of a SWidget it needs to have a Tag or TagMetadata with the appropriate plain text value
	 * <STextBlock> = <> represents types, in the case of a SWidget it should be the explicit type used in the SNew construction
	 *
	 * Hierarchy is represented by forward slashes
	 *
	 * / = a single forward slash represents that the next value must match a direct child of the element matched before it
	 * // = a double forward slash represents that the next value must match any descendant of the element matched before it
	 *
	 * Reference the AutomationDriver.spec.cpp expectations for additional examples of the syntax in use 
	 *
	 * @param Value - The path to use
	 * @return a locator which uses the specified path to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Path(const TCHAR* Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones matching the specified "path",
	 * starting from the Root element given to the function
	 *
	 * Path Example: "#Suite//Piano/Key//<STextBlock>"
	 *
	 * Path Syntax:
	 *
	 * #Suite = # represents that the following text is an explicit Id, in the case of a SWidget it needs to be tagged with the driver Id metadata
	 * Piano = plain text represents general tags, in the case of a SWidget it needs to have a Tag or TagMetadata with the appropriate plain text value
	 * <STextBlock> = <> represents types, in the case of a SWidget it should be the explicit type used in the SNew construction
	 *
	 * Hierarchy is represented by forward slashes
	 *
	 * / = a single forward slash represents that the next value must match a direct child of the element matched before it
	 * // = a double forward slash represents that the next value must match any descendant of the element matched before it
	 *
	 * Reference the AutomationDriver.spec.cpp expectations for additional examples of the syntax in use
	 *
	 * @param Root - The reference to the element where the search will be started from
	 * @param Value - The path to use
	 * @return a locator which uses the specified path to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Path(const FDriverElementRef& Root, const TCHAR* Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones matching the specified "path"
	 *
	 * Path Example: "#Suite//Piano/Key//<STextBlock>"
	 *
	 * Path Syntax:
	 *
	 * #Suite = # represents that the following text is an explicit Id, in the case of a SWidget it needs to be tagged with the driver Id metadata 
	 * Piano = plain text represents general tags, in the case of a SWidget it needs to have a Tag or TagMetadata with the appropriate plain text value
	 * <STextBlock> = <> represents types, in the case of a SWidget it should be the explicit type used in the SNew construction
	 *
	 * Hierarchy is represented by forward slashes
	 *
	 * / = a single forward slash represents that the next value must match a direct child of the element matched before it
	 * // = a double forward slash represents that the next value must match any descendant of the element matched before it
	 *
	 * Reference the AutomationDriver.spec.cpp expectations for additional examples of the syntax in use 
	 *
	 * @param Value - The path to use
	 * @return a locator which uses the specified path to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Path(const char* Value);

	/**
	 * Creates a new element locator that limits its discovered elements to ones matching the specified "path",
	 * starting from the Root element given to the function
	 *
	 * Path Example: "#Suite//Piano/Key//<STextBlock>"
	 *
	 * Path Syntax:
	 *
	 * #Suite = # represents that the following text is an explicit Id, in the case of a SWidget it needs to be tagged with the driver Id metadata
	 * Piano = plain text represents general tags, in the case of a SWidget it needs to have a Tag or TagMetadata with the appropriate plain text value
	 * <STextBlock> = <> represents types, in the case of a SWidget it should be the explicit type used in the SNew construction
	 *
	 * Hierarchy is represented by forward slashes
	 *
	 * / = a single forward slash represents that the next value must match a direct child of the element matched before it
	 * // = a double forward slash represents that the next value must match any descendant of the element matched before it
	 *
	 * Reference the AutomationDriver.spec.cpp expectations for additional examples of the syntax in use
	 *
	 * @param Root - The reference to the element where the search will be started from
	 * @param Value - The path to use
	 * @return a locator which uses the specified path to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Path(const FDriverElementRef& Root, const char* Value);

	/**
	 * Creates a new element locator that limits its discovered elements to the one under the cursors current position
	 * @return a locator which uses the cursor position to discover appropriate elements
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Cursor();
	
	/**
	 * Creates a new element locator that limits its discovered element to the one that's currently in keyboard focus
	 * @return a locator which uses the keyboard focus to discover the appropriate element
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> KeyboardFocus();

	/**
	 * Creates a new element locator that limits its discovered element to the one that's currently in user focus
	 * @return a locator which uses the user focus to discover the appropriate element
	 */
	static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> UserFocus(uint32 UserIndex = 0);

	class TextFilter
	{
	public:

		/**
		 * Creates a new element locator that limits the elements discovered by the root locator provided to the function
		 * to those containing the specified text substring.
		 *
		 * @param RootLocator - The reference to the element locator where the search will be started from
		 * @param Value - The text of the element to find
		 * @param SearchCase - Indicates whether the search is case sensitive or not ( defaults to ESearchCase::IgnoreCase )
		 * @param SearchDir - Indicates whether the search starts at the beginning or at the end ( defaults to ESearchDir::FromStart )
		 * @return a locator which uses the specified root locator and a text string to discover appropriate elements
		 */
		static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Contains(
			const FElementLocatorRef& RootLocator,
			const FString& Value,
			ESearchCase::Type SearchCase = ESearchCase::IgnoreCase,
			ESearchDir::Type SearchDir = ESearchDir::FromStart
		);

		/**
		 * Creates a new element locator that limits the elements discovered by the root locator provided to the function
		 * to those containing the specified text.
		 *
		 * @param RootLocator - The reference to the element locator where the search will be started from
		 * @param Value - The text of the element to find
		 * @param SearchCase - Indicates whether the search is case sensitive or not ( defaults to ESearchCase::CaseSensitive )
		 * @return a locator which uses the specified root locator and a text string to discover appropriate elements
		 */
		static UE_API TSharedRef<IElementLocator, ESPMode::ThreadSafe> Equals(
			const FElementLocatorRef& RootLocator,
			const FString& Value,
			ESearchCase::Type SearchCase = ESearchCase::CaseSensitive
		);
	};
};

#undef UE_API
