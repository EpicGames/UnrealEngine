// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/Timespan.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

#define UE_API AUTOMATIONDRIVER_API

class IElementLocator;

/**
 * Represents the state of an active wait action for the driver
 */
struct FDriverWaitResponse
{
public:

	enum class EState : uint8
	{
		PASSED,
		WAIT,
		FAILED,
	};

	/**
	 * @return a FDriverWaitResponse with a state of PASSED and a wait of zero
	 */
	static UE_API FDriverWaitResponse Passed();

	/**
	 * @return a FDriverWaitResponse with a state of WAIT and a wait of 0.5 seconds
	 */
	static UE_API FDriverWaitResponse Wait();

	/**
	 * @return a FDriverWaitResponse with a state of WAIT and a wait of the specified timespan
	 */
	static UE_API FDriverWaitResponse Wait(FTimespan Timespan);

	/**
	 * @return a FDriverWaitResponse with a state of FAILED and a wait of zero
	 */
	static UE_API FDriverWaitResponse Failed();

	UE_API FDriverWaitResponse(EState InState, FTimespan InNextWait);

	// How long the driver should wait before re-evaluating the wait condition again
	const FTimespan NextWait;

	// Whether the wait condition is completely finished or should be rescheduled again for execution
	const EState State;
};

DECLARE_DELEGATE_RetVal_OneParam(FDriverWaitResponse, FDriverWaitDelegate, const FTimespan& /*TotalWaitTime*/);
DECLARE_DELEGATE_RetVal(bool, FDriverWaitConditionDelegate);

/**
 * A fluent wrapper around timespan to enforce obvious differences between specified Timeout and Interval values for waits
 */
class FWaitTimeout
{
public:

	static UE_API FWaitTimeout InMilliseconds(double Value);
	static UE_API FWaitTimeout InSeconds(double Value);
	static UE_API FWaitTimeout InMinutes(double Value);
	static UE_API FWaitTimeout InHours(double Value);

	UE_API explicit FWaitTimeout(FTimespan InTimespan);

	const FTimespan Timespan;
};

/**
 * A fluent wrapper around timespan to enforce obvious differences between specified Timeout and Interval values for waits
 */
class FWaitInterval
{
public:

	static UE_API FWaitInterval InMilliseconds(double Value);
	static UE_API FWaitInterval InSeconds(double Value);
	static UE_API FWaitInterval InMinutes(double Value);
	static UE_API FWaitInterval InHours(double Value);

	UE_API explicit FWaitInterval(FTimespan InTimespan);

	const FTimespan Timespan;
};

/**
 * Represents a collection of fluent helper functions designed to make accessing and creating driver wait delegates easier
 */
class Until
{
public:

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers elements or if the specified timeout timespan elapses
	 */
	static UE_API FDriverWaitDelegate ElementExists(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers elements or if the specified timeout timespan elapses;
	 * The element locator is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate ElementExists(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers visible elements or if the specified timeout timespan elapses
	 */
	static UE_API FDriverWaitDelegate ElementIsVisible(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers visible elements or if the specified timeout timespan elapses;
	 * The element locator is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate ElementIsVisible(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	* Creates a new wait delegate which completes it's wait only if the specified element locator discovers hidden elements or if the specified timeout timespan elapses
	*/
	static UE_API FDriverWaitDelegate ElementIsHidden(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers hidden elements or if the specified timeout timespan elapses;
	 * The element locator is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate ElementIsHidden(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers interactable elements or if the specified timeout timespan elapses
	 */
	static UE_API FDriverWaitDelegate ElementIsInteractable(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers interactable elements or if the specified timeout timespan elapses;
	 * The element locator is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate ElementIsInteractable(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers a scrollable element whose scroll position is at the beginning
	 * or if the specified timeout timespan elapses
	 */
	static UE_API FDriverWaitDelegate ElementIsScrolledToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers a scrollable element whose scroll position is at the beginning
	 * or if the specified timeout timespan elapses
	 * The element locator is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate ElementIsScrolledToBeginning(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers a scrollable element whose scroll position is at the end
	 * or if the specified timeout timespan elapses
	 */
	static UE_API FDriverWaitDelegate ElementIsScrolledToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout);
	
	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers a scrollable element whose scroll position is at the end
	 * or if the specified timeout timespan elapses
	 * The element locator is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate ElementIsScrolledToEnd(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes its wait only if the specified element locator discovers an element focused by the default user or if the specified timeout timespan elapses
	 */
	static UE_API FDriverWaitDelegate ElementIsFocusedByKeyboard(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers an element focused by the default user or if the specified timeout timespan elapses;
	 * The element locator is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate ElementIsFocusedByKeyboard(const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes its wait only if the specified element locator discovers an element focused by the specified user or if the specified timeout timespan elapses
	 */
	static UE_API FDriverWaitDelegate ElementIsFocusedByUser(uint32 UserIndex, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified element locator discovers an element focused by the specified user or if the specified timeout timespan elapses;
	 * The element locator is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate ElementIsFocusedByUser(uint32 UserIndex, const TSharedRef<IElementLocator, ESPMode::ThreadSafe>& ElementLocator, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified condition returns true or if the specified timeout timespan elapses
	 */
	static UE_API FDriverWaitDelegate Condition(const TFunction<bool()>& Function, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified condition returns true or if the specified timeout timespan elapses
	 * The lambda is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate Condition(const TFunction<bool()>& Function, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified condition returns true or if the specified timeout timespan elapses
	 */
	static UE_API FDriverWaitDelegate Condition(const FDriverWaitConditionDelegate& Delegate, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which completes it's wait only if the specified condition returns true or if the specified timeout timespan elapses
	 * The delegate is only re-evaluated at the specified wait interval
	 */
	static UE_API FDriverWaitDelegate Condition(const FDriverWaitConditionDelegate& Delegate, FWaitInterval Interval, FWaitTimeout Timeout);

	/**
	 * Creates a new wait delegate which drives its state off the result of the specified lambda
	 */
	static UE_API FDriverWaitDelegate Lambda(const TFunction<FDriverWaitResponse(const FTimespan&)>& Value);
};

#undef UE_API
